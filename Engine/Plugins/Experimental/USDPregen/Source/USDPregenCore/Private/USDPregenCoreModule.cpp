// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenCoreModule.h"

#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

void FUSDPregenCoreModule::StartupModule()
{

}

void FUSDPregenCoreModule::ShutdownModule()
{

}

IMPLEMENT_MODULE(FUSDPregenCoreModule, USDPregenCore)