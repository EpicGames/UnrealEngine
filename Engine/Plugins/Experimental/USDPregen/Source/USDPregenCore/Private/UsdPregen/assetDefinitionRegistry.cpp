// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/assetDefinitionRegistry.h"

#include "UsdPregen/extAssetDefinition.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/hash.h"
#include "pxr/base/tf/instantiateSingleton.h"
#include "pxr/base/tf/singleton.h"
#include "pxr/base/tf/token.h"
#include "USDIncludesEnd.h"

#include <mutex>
#include <string>
#include <sstream>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE // for TF macros

PREGEN_NAMESPACE_USING_DIRECTIVE

TF_INSTANTIATE_SINGLETON(AssetDefinitionRegistry);

PREGEN_NAMESPACE_OPEN_SCOPE

namespace
{

// Converts a string identifier into the registry key.
pxr::TfToken
_MakeRegistryKey(const std::string& baseKey)
{
	return pxr::TfToken(baseKey);
}

// Emits a warning detailing conflicting fields if the two 
// definitions are not identical.
void
_CheckDefinitionConflicts(
	const ExtAssetDefinition& newDefn,
	const ExtAssetDefinition& existingDefn)
{
	if (ExtAssetDefinition::HasSameFields(newDefn, existingDefn))
	{
		return;
	}

	std::ostringstream msg;

	msg <<
		"New asset definition with (uid='" << newDefn.GetUniqueId() << "') has"
		" one or more fields that conflict with an existing definition with the"
		" same unique id. The new definition will not be added to the registry"
		" and the existing will be used instead.\n"
		" The conflicting fields are:\n";

	auto _CompareField = [&msg, &newDefn, &existingDefn]
		(auto&& getter, const char* label)
	{
		const auto& a = (newDefn.*getter)();
		const auto& b = (existingDefn.*getter)();
		if (a != b)
		{
			msg << " - " << label << ": "
				<< "'" << a << "' (new) vs "
				<< "'" << b << "' (existing)\n";
		}
	};

	// When comparing the asset identifier we only check the authored path, not
	// the resolved (absolute) file path. It's expected that asset resolvers
	// might resolve the same authored identifier to different underlying file
	// paths depending on the resolver context. So here we must assume that a
	// resolved path mismatch is intentional.
	//
	// When an asset resolver is not in use, the authored path is typically an
	// absolute path anyway. The default discovery plugin makes sure to convert
	// anchored relative authored paths ("./" and "../") to absolute paths to
	// ensure that the relative path that happened to be in use at the time the
	// asset was first discovered does not get captured into the definition.
	auto _CompareIdentifier = [&msg, &newDefn, &existingDefn]
		(auto&& getter, const char* label)
	{
		const pxr::SdfAssetPath& a = (newDefn.*getter)();
		const pxr::SdfAssetPath& b = (existingDefn.*getter)();
		if (a.GetAuthoredPath() != b.GetAuthoredPath())
		{
			msg << " - " << label << " (authored path): "
				<< a.GetAuthoredPath() << " (new) vs "
				<< b.GetAuthoredPath() << " (existing)\n";
		}
	};

	auto _CompareMetadata = [&msg, &newDefn, &existingDefn]() {
		static const pxr::VtDictionary empty;
		auto a = newDefn.GetMetadata();
		auto b = existingDefn.GetMetadata();
		const pxr::VtDictionary& da = a ? *a : empty;
		const pxr::VtDictionary& db = b ? *b : empty;
		if (da == db)
		{
			return;
		}

		msg << "- metadata: " << da << " (new) vs " << db << " (existing)\n";
	};

	_CompareField(&ExtAssetDefinition::GetName, "name");
	_CompareField(&ExtAssetDefinition::GetVersion, "version");
	_CompareIdentifier(&ExtAssetDefinition::GetIdentifier, "identifier");
	_CompareMetadata();

	TF_WARN("%s", msg.str().c_str());
}

} // anonymous namespace

// static
AssetDefinitionRegistry& AssetDefinitionRegistry::GetInstance()
{
	return pxr::TfSingleton<AssetDefinitionRegistry>::GetInstance();
}

const ExtAssetDefinition*
AssetDefinitionRegistry::AddDefinition(const ExtAssetDefinition& defn)
{
	if (std::string errorMsg; !defn.IsValid(&errorMsg))
	{
		TF_WARN(
			 "New asset definition (uid='%s') is invalid and cannot"
			 " be added to the registry - %s."
			, defn.GetUniqueId().c_str()
			, errorMsg.c_str()
		);

		return nullptr;
	}

	const pxr::TfToken key = _MakeRegistryKey(defn.GetUniqueId());

	std::lock_guard<std::mutex> lock(_mutex);

	auto [itr, inserted] = _entries.try_emplace(
		key, std::make_unique<ExtAssetDefinition>(defn));

	const ExtAssetDefinition* regDefn = itr->second.get();
	if (!inserted)
	{
		// An entry in the registry already exists, so check for conflicting
		// fields and emit a warning if needed.
		// Note that we currently return the definition in the registry even if it
		// doesn't match the provided definition. In the future we may want to signal
		// this more clearly to the user, so that they can make a choice of how to
		// handle the situation.
		_CheckDefinitionConflicts(defn, *regDefn);
	}

	return regDefn;
}

const ExtAssetDefinition*
AssetDefinitionRegistry::GetDefinition(const std::string& uniqueId) const
{
	const TfToken key = _MakeRegistryKey(uniqueId);

	std::lock_guard<std::mutex> lock(_mutex);

	auto itr = _entries.find(key);
	if (itr != _entries.end())
	{
		return itr->second.get();
	}

	return nullptr;
}

std::vector<const ExtAssetDefinition*>
AssetDefinitionRegistry::GetAllDefinitions() const
{
	std::lock_guard<std::mutex> lock(_mutex);

	std::vector<const ExtAssetDefinition*> values;
	values.reserve(_entries.size());

	for (const auto& [key, ptr] : _entries) {
		values.push_back(ptr.get());
	}

	return values;
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
