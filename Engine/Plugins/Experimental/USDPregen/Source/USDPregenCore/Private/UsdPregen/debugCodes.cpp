// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "debugCodes.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/debug.h"
#include "pxr/base/tf/registryManager.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_USING_DIRECTIVE

TF_REGISTRY_FUNCTION(TfDebug)
{
	TF_DEBUG_ENVIRONMENT_SYMBOL(
		PREGEN_ASSET,
		"Pregen asset and references."
	);

	TF_DEBUG_ENVIRONMENT_SYMBOL(
	    PREGEN_DISCOVERY,
		"Pregen default discovery plugin."
	);

	TF_DEBUG_ENVIRONMENT_SYMBOL(
		PREGEN_ITEM,
		"Pregen item and item scopes."
	);

	TF_DEBUG_ENVIRONMENT_SYMBOL(
		PREGEN_MANIFEST,
		"Pregen manifest."
	);

	TF_DEBUG_ENVIRONMENT_SYMBOL(
		PREGEN_PERMUTATION,
		"Pregen permutation processing."
	);

	TF_DEBUG_ENVIRONMENT_SYMBOL(
		PREGEN_REGISTRY,
		"Pregen registry."
	);

	TF_DEBUG_ENVIRONMENT_SYMBOL(
		PREGEN_TARGET,
		"Pregen target generation."
	);

	TF_DEBUG_ENVIRONMENT_SYMBOL(
		PREGEN_TRACKING,
		"Pregen tracking."
	);
}

#endif // USE_USD_SDK
