// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/debug.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE

TF_DEBUG_CODES(

	PREGEN_ASSET,
	PREGEN_DISCOVERY,
	PREGEN_ITEM,
	PREGEN_MANIFEST,
	PREGEN_PERMUTATION,
	PREGEN_REGISTRY,
	PREGEN_TARGET,
	PREGEN_TRACKING
);

PXR_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
