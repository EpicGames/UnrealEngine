// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "debugCodes.h"

//#ifndef PREGEN_ENABLE_DEBUG_MACROS
//#error "This file must only be included from a .cpp"
//#endif

#define DEBUG_ASSET(fmt, ...) \
	TF_DEBUG(PREGEN_ASSET).Msg("[DEBUG_ASSET] " fmt, ##__VA_ARGS__)

#define DEBUG_DISCOVERY(fmt, ...) \
	TF_DEBUG(PREGEN_DISCOVERY).Msg("[DEBUG_DISCOVERY] " fmt, ##__VA_ARGS__)

#define DEBUG_ITEM(fmt, ...) \
	TF_DEBUG(PREGEN_ITEM).Msg("[DEBUG_ITEM] " fmt, ##__VA_ARGS__)

#define DEBUG_MANIFEST(fmt, ...) \
	TF_DEBUG(PREGEN_MANIFEST).Msg("[DEBUG_MANIFEST] " fmt, ##__VA_ARGS__)

#define DEBUG_PERMUTATION(fmt, ...) \
	TF_DEBUG(PREGEN_PERMUTATION).Msg("[DEBUG_PERMUTATION] " fmt, ##__VA_ARGS__)

#define DEBUG_REGISTRY(fmt, ...) \
	TF_DEBUG(PREGEN_REGISTRY).Msg("[DEBUG_REGISTRY] " fmt, ##__VA_ARGS__)

#define DEBUG_TARGET(fmt, ...) \
	TF_DEBUG(PREGEN_TARGET).Msg("[DEBUG_TARGET] " fmt, ##__VA_ARGS__)

#define DEBUG_TRACKING(fmt, ...) \
	TF_DEBUG(PREGEN_TRACKING).Msg("[DEBUG_TRACKING] " fmt, ##__VA_ARGS__)

#endif // USE_USD_SDK
