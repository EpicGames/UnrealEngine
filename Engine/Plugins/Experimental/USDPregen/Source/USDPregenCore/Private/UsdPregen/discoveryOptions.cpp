// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/discoveryOptions.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/enum.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/registryManager.h"
#include "USDIncludesEnd.h"

#include <cstddef> // std::size_t
#include <sstream>

PREGEN_NAMESPACE_USING_DIRECTIVE

PXR_NAMESPACE_OPEN_SCOPE

TF_REGISTRY_FUNCTION(TfEnum)
{
	TF_ADD_ENUM_NAME(DiscoveryMode::AllPermutations);
	TF_ADD_ENUM_NAME(DiscoveryMode::ComposedPermutationOnly);

	TF_ADD_ENUM_NAME(IdentifierFallbackMode::None);
	TF_ADD_ENUM_NAME(IdentifierFallbackMode::FirstDirectReferenceOrPayload);

	TF_ADD_ENUM_NAME(VersionFallbackMode::None);
	TF_ADD_ENUM_NAME(VersionFallbackMode::LayerStackFilesAndTimestamps);
	TF_ADD_ENUM_NAME(VersionFallbackMode::ResolvedLayerStackFilesAndTimestamps);
}

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

bool
DiscoveryOptions::operator==(const DiscoveryOptions& rhs) const
{
	return
		discoveryMode == rhs.discoveryMode &&
		discoveryPluginName == rhs.discoveryPluginName &&
		definitionPrefix == rhs.definitionPrefix &&
		initialPath == rhs.initialPath &&
		purposes == rhs.purposes &&
		excludeVariantSets == rhs.excludeVariantSets &&
		assetIdentifierFallback == rhs.assetIdentifierFallback &&
		assetVersionFallback == rhs.assetVersionFallback &&
		traversalPredicate == rhs.traversalPredicate;
}

bool
DiscoveryOptions::operator!=(const DiscoveryOptions& rhs) const
{
	return !(*this == rhs);
}

std::string
DiscoveryOptions::DumpToString() const
{
	std::ostringstream os;

	os << "DiscoveryOptions {\n";

	os << "  discoveryMode: "
	   << pxr::TfEnum::GetName(discoveryMode) << "\n";

	os << "  discoveryPluginName: \""
	   << discoveryPluginName << "\"\n";

	os << "  definitionPrefix: \""
	   << definitionPrefix << "\"\n";

	os << "  initialPath: "
	   << initialPath.GetString() << "\n";

	os << "  purposes: [";

	for (std::size_t i = 0; i < purposes.size(); ++i)
	{
		if (i > 0) os << ", ";
		os << purposes[i].GetString();
	}
	os << "]\n";

	os << "  excludeVariantSets: [";

	for (std::size_t i = 0; i < excludeVariantSets.size(); ++i)
	{
		if (i > 0) os << ", ";
		os << excludeVariantSets[i].GetString();
	}
	os << "]\n";

	os << "  assetIdentifierFallback: "
	   << pxr::TfEnum::GetName(assetIdentifierFallback) << "\n";

	os << "  assetVersionFallback: "
	   << pxr::TfEnum::GetName(assetVersionFallback) << "\n";

	os << "  traversalPredicate: "
	   << (traversalPredicate == pxr::UsdPrimDefaultPredicate
		   ? "UsdPrimDefaultPredicate"
		   : "<custom>")
	   << "\n";

	os << "}";

	return os.str();
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
