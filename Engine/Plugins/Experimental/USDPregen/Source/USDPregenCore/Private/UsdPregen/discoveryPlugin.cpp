// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/discoveryPlugin.h"

#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/permutationOps.h"
#include "UsdPregen/primPermutation.h"

#include "persistentHasher.h"
#include "util.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/pathUtils.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/tf/token.h"
#include "pxr/base/vt/dictionary.h"
#include "pxr/base/vt/value.h"
#include "pxr/usd/sdf/assetPath.h"
#include "pxr/usd/sdf/layer.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/usd/modelAPI.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usd/variantSets.h"
#include "pxr/usd/usdGeom/boundable.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdLux/boundableLightBase.h"
#include "pxr/usd/usdLux/nonboundableLightBase.h"
#include "pxr/usd/usdShade/material.h"
#include "pxr/usd/usdShade/shader.h"
#include "pxr/usd/usdSkel/root.h"
#include "pxr/usd/usdSkel/skeleton.h"
#include "pxr/usd/usdSkel/animation.h"
#include "USDIncludesEnd.h"

#include <algorithm>
#include <cinttypes>
#include <memory>
#include <mutex>
#include <optional>
#include <string>
#include <unordered_map>
#include <vector>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE

PREGEN_NAMESPACE_OPEN_SCOPE

const pxr::TfToken DiscoveryPluginTokens::reservedMetadataPrefix{"_usdpregen:"};
const pxr::TfToken DiscoveryPluginTokens::definitionPrefix{"_usdpregen:definitionPrefix"};
const pxr::TfToken DiscoveryPluginTokens::initialPrim{"_usdpregen:initialPrim"};

const pxr::TfToken DefaultCategoryTokens::meshes{"meshes"};
const pxr::TfToken DefaultCategoryTokens::materials{"materials"};
const pxr::TfToken DefaultCategoryTokens::shaders{"shaders"};
const pxr::TfToken DefaultCategoryTokens::skeletons{"skeletons"};
const pxr::TfToken DefaultCategoryTokens::skelAnimations{"skelAnimations"};
const pxr::TfToken DefaultCategoryTokens::skelRoots{"skelRoots"};
const pxr::TfToken DefaultCategoryTokens::lights{"lights"};
const pxr::TfToken DefaultCategoryTokens::boundables{"boundables"};
const pxr::TfToken DefaultCategoryTokens::noCategory;

namespace internal
{

struct _VersionCache
{
	std::unordered_map<pxr::SdfAssetPath,
		               std::string,
					   pxr::TfHash> assetToVersionMap;
	std::mutex mutex;
};

} // namespace internal

namespace
{

// Encodes a uint64 as base36 ([0-9a-z], lowercase only) without padding.
// Produces at most 13 characters (36^13 > 2^64).
//
// Lowercase-only so the encoded value is stable across case-insensitive
// filesystems (NTFS, APFS-by-default): a base62 encoding could produce
// pairs like "aB" and "Ab" that collide on Windows but not on Linux.
std::string
EncodeUint64Base36(std::uint64_t value)
{
	static constexpr char alphabet[] = "0123456789abcdefghijklmnopqrstuvwxyz";
	static constexpr std::uint64_t base = 36;

	if (value == 0)
	{
		return "0";
	}

	// uint64_t max in base36 fits in 13 digits (36^13 > 2^64 > 36^12).
	std::string result;
	result.reserve(13);
	while (value > 0)
	{
		result.push_back(alphabet[value % base]);
		value /= base;
	}

	// Digits were emitted least-significant-first.
	std::reverse(result.begin(), result.end());
	return result;
}

// Attempts to populate a missing asset identifier using fallback discovery
// rules defined by the given options.
//
// The incoming identifier is always cleared before fallback resolution is
// attempted.
//
// For the initial discovery prim, the identifier is extracted from the first
// defining spec in the local layer stack. For all other prims, the identifier
// is extracted from the first direct reference or payload.
//
// If fallback resolution succeeds, relative authored paths beginning with
// "./" or "../" are converted to absolute authored paths using the resolved
// path. However, unanchored search-relative paths may be preserved if they
// appear to resolve to the same trailing path components as the resolved
// asset path.
//
// Returns true if a fallback identifier was successfully determined.
bool
_GetIdentifierFallback(
	const pxr::UsdPrim& prim,
	const pxr::SdfPath& initialPath,
	pxr::SdfAssetPath& identifier,
	const DiscoveryOptions& options)
{
	using namespace internal::util;

	// First capture the incoming authored path, if any. We may use it later
	// to preserve unanchored search-relative paths.
	const std::string inAuthoredPath = identifier.GetAuthoredPath();

	// Clear the input identifier before attempting fallback resolution.
	identifier = pxr::SdfAssetPath{};

	const auto fallbackMode = options.assetIdentifierFallback;
	if (fallbackMode == IdentifierFallbackMode::None)
	{
		return false;
	}

	TF_VERIFY(fallbackMode == IdentifierFallbackMode::FirstDirectReferenceOrPayload);

	// Special case for the initial prim, since searching for a reference
	// is unlikely to yield the correct identifier.
	if (prim.GetPrimPath() == initialPath)
	{
		DEBUG_DISCOVERY(
		    "Initial asset prim <%s> is missing authored AssetInfo identifier "
			"metadata - attempting to extract from local layer stack.\n"
			, prim.GetPrimPath().GetText()
		);

		identifier = GetAssetFromFirstDefiningSpec(prim);
	}
	// Otherwise search for the first direct reference or payload.
	else
	{
		DEBUG_DISCOVERY(
		    "Asset prim <%s> is missing authored AssetInfo identifier metadata "
		    "- attempting to extract from direct reference or payload.\n"
			, prim.GetPrimPath().GetText()
		);

		identifier = GetAssetFromFirstDirectReferenceOrPayload(prim);

		// Handle cases where the asset layer has no default prim: Our initialPath
		// will fall back to the pseudoroot path, and we'll fall in this branch,
		// but our asset prim may not really have any reference or payload
		if (identifier == pxr::SdfAssetPath{})
		{
			DEBUG_DISCOVERY(
				"Asset prim <%s> has no direct reference or payload. Attempting "
				"to extract from local layer stack.\n",
				prim.GetPrimPath().GetText()
			);

			identifier = GetAssetFromFirstDefiningSpec(prim);
		}
	}

	if (identifier == pxr::SdfAssetPath{})
	{
		TF_WARN(
		    "Failed to deduce fallback identifier for asset prim <%s>"
			, prim.GetPrimPath().GetText()
		);

		return false;
	}

	const std::string& authoredPath = identifier.GetAuthoredPath();
	const std::string& resolvedPath = identifier.GetResolvedPath();

	// Preserve the incoming authored path if it appears to be an unanchored
	// search-relative path that resolved to the same trailing file and path
	// components.
	//
	// For example, Kitchen_set asset identifiers may be authored as:
	//
	//     @assets/Ball/Ball.usd@
	//
	// Such paths resolve correctly when opened from Kitchen_set.usd (which
	// resides beside the "assets" directory), but do not resolve when opening
	// Ball.usd directly.
	//
	// The test below allows us to preserve the unanchored authored path, rather
	// than overwrite it with an absolute path.

	const std::string normAuthored = pxr::TfNormPath(inAuthoredPath);
	const std::string normResolved = pxr::TfNormPath(resolvedPath);

	if (!inAuthoredPath.empty() &&
		// unanchored paths cannot start with "." or ".."
		!pxr::TfStringStartsWith(inAuthoredPath, ".") &&
		// out path must not contain ":" (either a Windows drive letter or URI)
		!pxr::TfStringContains(inAuthoredPath, ":") &&
		// ignore absolute paths
	    !pxr::TfStringStartsWith(inAuthoredPath, "/") &&
		!pxr::TfStringStartsWith(inAuthoredPath, "\\") &&
		// the resolved and authored path must end the same.
		pxr::TfStringEndsWith(normResolved, normAuthored) &&
		// boundary check to ensure it's the start of the string or preceded by a slash
		(normResolved.length() == normAuthored.length() ||
		 normResolved[normResolved.length() - normAuthored.length() - 1] == '/'))
	{
		identifier.SetAuthoredPath(inAuthoredPath);
	}
	// Otherwise convert anchored relative authored paths to absolute paths,
	// since there is not much point storing relative paths in the registry
	// or manifests.
	else if (pxr::TfStringStartsWith(authoredPath, "."))
	{
		identifier.SetAuthoredPath(resolvedPath);
	}

	return true;
}

// Attempts to generate a fallback asset version string using information
// derived from the stage's local layer stack.
//
// The generated version always combines the latest layer modification
// timestamp with a 64-bit hash. The hash incorporates the modification
// time of each layer plus either layer identifiers
// (LayerStackFilesAndTimestamps) or resolved real paths
// (ResolvedLayerStackFilesAndTimestamps), depending on the configured
// fallback mode.
//
// Returns an empty string if version fallback generation is disabled.
std::string
_GetVersionFallback(const pxr::UsdStageConstRefPtr& stage,
					const DiscoveryOptions& options)
{
	using namespace internal::util;

	const VersionFallbackMode versionFallback = options.assetVersionFallback;
	if (versionFallback == VersionFallbackMode::None)
	{
		return {};
	}

	const bool hashRealPaths = versionFallback
		== VersionFallbackMode::ResolvedLayerStackFilesAndTimestamps;

	const std::pair<std::string, std::uint64_t> timestampAndHash
		= GetLocalLayerStackTimestampAndHash(stage, hashRealPaths);

	return TfStringPrintf("v%s_%016" PRIx64,
		timestampAndHash.first.c_str(),
		timestampAndHash.second);
}

// Attempts to generate a fallback asset version string for the given asset
// identifier.
//
// The generated version is cached per identifier to avoid repeatedly opening
// stages and recomputing local layer stack hashes and timestamps.
//
// Returns an empty string if the identifier cannot be resolved to a local
// stage or if version fallback generation fails.
std::string
_GetVersionFallback(const pxr::SdfAssetPath& identifier,
	                internal::_VersionCache& cache,
	                const DiscoveryOptions& options)
{
	const std::string& usdFilePath = identifier.GetResolvedPath();

	if (!TF_VERIFY(!usdFilePath.empty()))
	{
		return {};
	}

	{
		std::scoped_lock lock(cache.mutex);
		if (const auto& itr = cache.assetToVersionMap.find(identifier);
			itr != cache.assetToVersionMap.end())
		{
			return itr->second;
		}
	}

	// Open the stage fully masked in order to generate a version from the names
	// and timestamps of the local layer stack. This allows us to detect some
	// degree of changes to the contributing files, without recursively pulling
	// in references and payloads.
	std::string version;

	if (const pxr::UsdStageRefPtr stage = UsdStage::OpenMasked(usdFilePath, {}))
	{
		version = _GetVersionFallback(stage, options);
	}
	else
	{
		TF_WARN(
		    "Failed to open stage for identifier @%s@ while computing version "
			"fallback for asset. Version string may be missing or incomplete."
			, identifier.GetAssetPath().c_str()
	    );
	}

	std::scoped_lock lock(cache.mutex);

	auto [itr, _] = cache.assetToVersionMap.emplace(
					    identifier,
					    std::move(version)
					);

	return itr->second;
}

void
_CopyDictionaryRecursive(const pxr::VtDictionary& src, pxr::VtDictionary& dst)
{
	for (const auto& [key, value] : src)
	{
		if (value.IsHolding<pxr::VtDictionary>())
		{
			// If the value is a dictionary, create a nested entry and recurse.
			pxr::VtDictionary nestedDst;
			_CopyDictionaryRecursive(value.UncheckedGet<pxr::VtDictionary>(), nestedDst);
			dst[key] = nestedDst;
		}
		else
		{
			// Otherwise, just copy the value
			dst[key] = value;
		}
	}
}

void
_AttachCustomAssetInfoMetadata(const pxr::UsdPrim& prim, pxr::VtDictionary& dict)
{
	if (!prim || prim.IsPseudoRoot())
	{
		return;
	}

	const pxr::UsdModelAPI modelApi{ prim };
	pxr::VtDictionary assetInfo;
	if (!modelApi.GetAssetInfo(&assetInfo))
	{
		return;
	}

	// Remove the built-in asset info keys (name, version, etc)
	for (const pxr::TfToken& token : pxr::UsdModelAPIAssetInfoKeys->allTokens)
	{
		assetInfo.erase(token);
	}

	// Copy the remaining custom fields.
	_CopyDictionaryRecursive(assetInfo, dict);
}

} // anonymous namespace

DiscoveryPlugin::DiscoveryPlugin(const DiscoveryOptions& options)
	: _options(options)
{
	_excludeVariantSets.reserve(_options.excludeVariantSets.size());
	_excludeVariantSets.insert(
	    _options.excludeVariantSets.begin(),
		_options.excludeVariantSets.end());

	if (_excludeVariantSets.count(TfToken("*")))
	{
		_excludeAllVariantSets = true;
	}
}

DiscoveryPlugin::~DiscoveryPlugin() = default;

// virtual
bool
DiscoveryPlugin::Initialize(const pxr::UsdStageRefPtr& stage)
{
	_initialPath = [&stage]()
	{
		// Prefer the stage default prim if there is one
		if (stage->HasDefaultPrim())
		{
			return stage->GetDefaultPrim().GetPrimPath();
		}

		// Otherwise start from the pseudo root.
		return pxr::SdfPath::AbsoluteRootPath();
	}();

	DEBUG_DISCOVERY(
	    "Default discovery plugin initialized with options ...\n%s\n"
		, _options.DumpToString().c_str()
	);

	_versionCache = std::make_unique<internal::_VersionCache>();

	return true;
}

// virtual
pxr::SdfPath
DiscoveryPlugin::GetInitialPath() const
{
	return _initialPath;
}

// virtual
bool
DiscoveryPlugin::IsAsset(const pxr::UsdPrim& prim) const
{
	std::string name;

	const pxr::UsdModelAPI modelApi{ prim };
	const bool hasAssetName = modelApi.GetAssetName(&name);

	return hasAssetName;
}

// virtual
bool
DiscoveryPlugin::PrunePrim(const pxr::UsdPrim& prim) const
{
	return false;
}

// virtual
ExtAssetDefinition
DiscoveryPlugin::GetAssetDefinition(const pxr::UsdPrim& prim) const
{
	if (!prim)
	{
		return {};
	}

	// A custom plugin can return true for IsAsset() when it encounters the
	// pseudo root, and then defer to this function to produce a definition,
	// and so we must support this case.
	if (prim.IsPseudoRoot())
	{
		return CreateRootDefinition(prim);
	}

	// The fields we need for a valid definition. Name and identifier cannot be
	// empty. Version is optional, and so can be empty, as can metadata.
	std::string name;
	std::string version;
	pxr::SdfAssetPath identifier;
	pxr::VtDictionary metadata;

	const pxr::UsdModelAPI modelApi {prim};

	// Get the asset name (the prior IsAsset() call should
	// guarantee we have a name)
	const bool hasAssetName = modelApi.GetAssetName(&name);
	if (!TF_VERIFY(hasAssetName))
	{
		return {};
	}

	// Add the definition prefix if set.
	if (const std::string prefix = GetOptions().definitionPrefix;
		!prefix.empty())
	{
		name = prefix + name;
		metadata[DiscoveryPluginTokens::definitionPrefix] = prefix;
	}

	// Attempt to get the explicit, authored asset identifier. If this fails
	// try using the fallback strategy.
	bool hasAssetIdentifier = modelApi.GetAssetIdentifier(&identifier);
	if (!hasAssetIdentifier || identifier.GetResolvedPath().empty())
	{
		hasAssetIdentifier = _GetIdentifierFallback(
							     prim,
								 _initialPath,
								 identifier,
								 GetOptions()
							 );
	}

	// Get the version. An empty version string is valid/supported.
	if (!modelApi.GetAssetVersion(&version) && hasAssetIdentifier)
	{
		version = _GetVersionFallback(identifier, *_versionCache.get(), GetOptions());
	}

	if (hasAssetIdentifier)
	{
		_AttachCustomAssetInfoMetadata(prim, metadata);

		return { name, version, identifier, metadata };
	}

	TF_WARN(
	    "Failed to get valid asset identifier while generating "
		"external asset definition for prim <%s>"
	    , prim.GetPath().GetText()
	);

	return {};
}

// virtual
PrimPermutation
DiscoveryPlugin::GetCurrentPrimPermutation(const pxr::UsdPrim& prim) const
{
	const pxr::SdfPath& primPath = prim.GetPrimPath();

	if (_excludeAllVariantSets)
	{
		return { primPath };
	}

	PermutationOpVector variantOps;

	pxr::UsdVariantSets variantSets = prim.GetVariantSets();
	for (const std::string& variantSetName : variantSets.GetNames())
	{
		if (_excludeVariantSets.count(pxr::TfToken{variantSetName}))
		{
			DEBUG_DISCOVERY(
		        "Excluding variant set (%s) on prim <%s>\n"
				, variantSetName.c_str()
				, primPath.GetText()
			);
			continue;
		}

		const std::string variantName
			= variantSets.GetVariantSelection(variantSetName);
		if (!variantName.empty())
		{
			variantOps.push_back(std::make_shared
				<UsdVariantPermutationOp>(variantSetName, variantName));
		}
	}

	if (!variantOps.empty())
	{
		DEBUG_DISCOVERY(
		    "Got (%zu) variant ops for current permutation on prim <%s>\n"
			, variantOps.size()
			, primPath.GetText()
		);
	}

	return { primPath, variantOps };
}

// virtual
std::vector<PrimPermutation>
DiscoveryPlugin::GetPrimPermutations(const pxr::UsdPrim& prim) const
{
	const pxr::SdfPath& primPath = prim.GetPrimPath();

	if (_excludeAllVariantSets)
	{
		return { primPath };
	}

	std::vector<PermutationOpVector> ops;

	pxr::UsdVariantSets variantSets = prim.GetVariantSets();
	for (const std::string& variantSetName : variantSets.GetNames())
	{
		if (_excludeVariantSets.count(pxr::TfToken{variantSetName}))
		{
			DEBUG_DISCOVERY(
			    "Excluding variant set (%s) on prim <%s>\n"
				, variantSetName.c_str()
				, primPath.GetText()
			);
			continue;
		}

		// Note: if there's an authored selection it means the variant set has
		// already been discovered, and so we shouldn't supply it a second time.
		const std::string currentSelection
			= variantSets.GetVariantSelection(variantSetName);

		if (!currentSelection.empty())
		{
			DEBUG_DISCOVERY("Ignoring variantset (%s) with currently "
				"active selection (%s)\n", variantSetName.c_str(),
				currentSelection.c_str());
			continue;
		}

		PermutationOpVector variantOps;
		pxr::UsdVariantSet variantSet = variantSets.GetVariantSet(variantSetName);
		for (const std::string& variantName : variantSet.GetVariantNames())
		{
			variantOps.push_back(std::make_shared
				<UsdVariantPermutationOp>(variantSetName, variantName)
			);
		}

		if (!variantOps.empty())
		{
			ops.push_back(variantOps);
		}
	}

	if (ops.empty())
	{
		return {};
	}

	const auto products = internal::util::CartesianProduct(ops);
	std::vector<PrimPermutation> result;
	result.reserve(products.size());
	for (const auto& product : products)
	{
		result.emplace_back(primPath, product);
	}

	DEBUG_DISCOVERY(
	    "Got (%zu) permutations for prim <%s>\n"
		, result.size()
		, primPath.GetText()
	);

	return result;
}

// virtual
ExtAssetDefinition
DiscoveryPlugin::CreateRootDefinition(const pxr::UsdPrim& prim) const
{
	if (!TF_VERIFY(prim))
	{
		return {};
	}

	const pxr::SdfLayerRefPtr layer = prim.GetStage()->GetRootLayer();
	const pxr::SdfAssetPath identifier{
		layer->GetIdentifier(),
		layer->GetRealPath()
	};

	const bool haveRealPath = !layer->GetRealPath().empty();

	pxr::VtDictionary metadata;
	metadata[DiscoveryPluginTokens::initialPrim] = prim.GetPrimPath().GetString();

	std::string name;
	if (layer->IsAnonymous())
	{
		TF_WARN(
		     "Anonymous root layer (%s) detected) - root-level scene"
			 "description will not be associated with a persistent file."
			 , identifier.GetAssetPath().c_str()
		);
		name = identifier.GetAssetPath();
	}
	else
	{
		internal::PersistentHasher hasher;

		const std::string baseName = pxr::TfStringGetBeforeSuffix(haveRealPath
			? pxr::TfGetBaseName(identifier.GetResolvedPath())
			: identifier.GetAuthoredPath());

		if (haveRealPath)
		{
			// Hash the full identifier as we have removed the extension
			// from the name, so we need it to be part of the hash or else
			// ball.usda and ball.usdc will collide
			hasher.AddFilePath(identifier.GetResolvedPath());
		}

		// If the root prim is not the pseudo root, add a hash of the
		// path to the definition name.
		if (!prim.IsPseudoRoot())
		{
			hasher.AddString(prim.GetPrimPath().GetString());
		}

		// Lastly, incorporate the prefix if set.
		const std::string prefix = GetOptions().definitionPrefix;
		if (!prefix.empty())
		{
			metadata[DiscoveryPluginTokens::definitionPrefix] = prefix;
		}

		// Base36-encode the hash so it fits in at most 13 chars (vs 20 for
		// decimal): asset paths are already close to Windows' 260-char limit.
		// Lowercase-only encoding so the resulting path is stable on
		// case-insensitive filesystems like NTFS.
		name = prefix + baseName + "_"
			+ EncodeUint64Base36(hasher.GetHash64());
	}

	const std::string version = _GetVersionFallback(prim.GetStage(), GetOptions());

	_AttachCustomAssetInfoMetadata(prim, metadata);

	return ExtAssetDefinition{name, version, identifier, metadata};
};

// virtual
pxr::TfToken
DiscoveryPlugin::GetContentCategoryForPrim(
	const pxr::UsdPrim& prim,
	const ExtAssetDefinition* defn) const
{
	if (prim.IsA<pxr::UsdGeomMesh>())
		return DefaultCategoryTokens::meshes;

	if (prim.IsA<pxr::UsdShadeMaterial>())
		return DefaultCategoryTokens::materials;

	if (prim.IsA<pxr::UsdShadeShader>())
		return DefaultCategoryTokens::shaders;

	if (prim.IsA<pxr::UsdSkelRoot>())
		return DefaultCategoryTokens::skelRoots;

	if (prim.IsA<pxr::UsdSkelSkeleton>())
		return DefaultCategoryTokens::skeletons;

	if (prim.IsA<pxr::UsdSkelAnimation>())
		return DefaultCategoryTokens::skelAnimations;

	if (prim.IsA<pxr::UsdLuxBoundableLightBase>() ||
		prim.IsA<pxr::UsdLuxNonboundableLightBase>())
		return DefaultCategoryTokens::lights;

	if (prim.IsA<pxr::UsdGeomBoundable>())
		return DefaultCategoryTokens::boundables;

	return DefaultCategoryTokens::noCategory;
}

// virtual
std::optional<pxr::SdfPathSet>
DiscoveryPlugin::GetOverrideCandidatePrimPaths(
   const pxr::UsdPrim& assetPrim,
   const ExtAssetDefinition* defn) const
{
	return std::nullopt;
}

const DiscoveryOptions&
DiscoveryPlugin::GetOptions() const
{
	return _options;
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
