// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/discoveryPluginRegistry.h"

#include "UsdPregen/discoveryOptions.h"
#include "UsdPregen/discoveryPlugin.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/instantiateSingleton.h"
#include "pxr/base/tf/singleton.h"
#include "pxr/base/tf/registryManager.h"
#include "USDIncludesEnd.h"

#include <string>
#include <memory>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE // for TF macros

PREGEN_NAMESPACE_USING_DIRECTIVE

TF_INSTANTIATE_SINGLETON(DiscoveryPluginRegistry);

PREGEN_NAMESPACE_OPEN_SCOPE

bool
DiscoveryPluginRegistry::RegisterFactory(const std::string& pluginName, FactoryFn fn)
{
	if (pluginName.empty() || !fn)
	{
		TF_WARN("Failed to register factory with invalid name and/or callback");
		return false;
	}

	DEBUG_REGISTRY(
		"Registering discovery plugin (%s)\n"
		, pluginName.c_str()
	);

	auto [itr, inserted] = _registry.emplace(pluginName, fn);
	// TODO unloading/reloading from python conveniences
	return inserted;
}

DiscoveryPluginRefPtr
DiscoveryPluginRegistry::Create(const DiscoveryOptions& options) const
{
	pxr::TfRegistryManager::GetInstance().SubscribeTo<DiscoveryPluginRegistry>();

	const std::string& pluginName = options.discoveryPluginName;

	auto itr = _registry.find(pluginName);
	if (itr == _registry.end() || itr->second == nullptr)
	{
		TF_WARN(
			"Failed to find a valid factory function for requested "
			"pregen discovery plugin (%s)\n"
			, pluginName.c_str()
		);

		return nullptr;
	}

	DEBUG_REGISTRY(
		"Creating discovery plugin instance (%s)\n"
		, pluginName.c_str()
	);

	return DiscoveryPluginRefPtr(itr->second(options));
}

// static
DiscoveryPluginRegistry& DiscoveryPluginRegistry::GetInstance()
{
	return TfSingleton<DiscoveryPluginRegistry>::GetInstance();
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
