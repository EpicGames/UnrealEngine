// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/extAssetDefinition.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/pathUtils.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/usd/sdf/assetPath.h"
#include "pxr/usd/sdf/path.h"
#include "USDIncludesEnd.h"

#include <string>
#include <optional>
#include <variant>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE // for TF macros

PREGEN_NAMESPACE_OPEN_SCOPE

namespace
{

// Matches USD/SDF prim-name rules for the leading character: an ASCII alpha
// or an underscore
bool _StartsWithValidIdChar(const std::string& str)
{
	if (str.empty())
	{
		return false;
	}

	const char c = str.front();

	return (c >= 'A' && c <= 'Z') ||
		   (c >= 'a' && c <= 'z') ||
		   c == '_';
}

} // anonymous namespace

ExtAssetDefinition::ExtAssetDefinition() = default;

ExtAssetDefinition::ExtAssetDefinition(
	const std::string& name,
	const std::string& version,
	const pxr::SdfAssetPath& identifier)
	: ExtAssetDefinition(
		name,
		version,
		identifier,
		/*customUniqueId*/ std::string{},
		/*metadata*/ std::nullopt)
{
}

ExtAssetDefinition::ExtAssetDefinition(
	const std::string& name,
	const std::string& version,
	const pxr::SdfAssetPath& identifier,
	const std::string& customUniqueId)
	: ExtAssetDefinition(
		name,
		version,
		identifier,
		customUniqueId,
		/*metadata*/ std::nullopt)
{
}

ExtAssetDefinition::ExtAssetDefinition(
	const std::string& name,
	const std::string& version,
	const pxr::SdfAssetPath& identifier,
	const pxr::VtDictionary& metadata)
	: ExtAssetDefinition(
		name,
		version,
		identifier,
		/*customUniqueId*/ std::string{},
		metadata)
{
}

ExtAssetDefinition::ExtAssetDefinition(
	const std::string& name,
	const std::string& version,
	const pxr::SdfAssetPath& identifier,
	const std::string& customUniqueId,
	const pxr::VtDictionary& metadata)
	: ExtAssetDefinition(
		name,
		version,
		identifier,
		customUniqueId,
		std::optional<VtDictionary>{metadata})
{
}

// private
ExtAssetDefinition::ExtAssetDefinition(
	const std::string& name,
	const std::string& version,
	const pxr::SdfAssetPath& identifier,
	const std::string& customUniqueId,
	const std::optional<pxr::VtDictionary>& metadata)
	: _name(name)
	, _version(version)
	, _identifier(identifier)
	, _uniqueId(customUniqueId.empty()
		? _UniqueId{ _DefaultUniqueId{ MakeDefaultUniqueId(name, version) } }
		: _UniqueId{ _CustomUniqueId{ customUniqueId } })
	, _metadata(metadata)
{
	// Normalize the resolved path, since it will get compared with layer stack
	// paths from Pcp nodes, that won't match if the separators are inconsistent
	if (!_identifier.GetResolvedPath().empty())
	{
		_identifier.SetResolvedPath(
		    pxr::TfNormPath(_identifier.GetResolvedPath())
		);

		// If the authored path is a relative path, replace it with the absolute
		// resolved path. This avoids conflicting definitions simply because the
		// same layer is discovered via different relative paths.
		if (pxr::TfStringStartsWith(_identifier.GetAuthoredPath(), "."))
		{
			_identifier.SetAuthoredPath(_identifier.GetResolvedPath());
		}
	}

	DEBUG_ASSET("Constructing %s ExtAssetDefinition ...\n"
		".. name: %s\n"
		".. version: %s\n"
		".. identifier: (authoredPath=%s, resolvedPath=%s)\n"
		".. uniqueId: %s\n"
		, HasCustomUniqueId() ? "custom" : "default"
		, GetName().c_str()
		, GetVersion().c_str()
		, GetIdentifier().GetAuthoredPath().c_str()
		, GetIdentifier().GetResolvedPath().c_str()
		, GetUniqueId().c_str()
	);
}

bool
ExtAssetDefinition::operator==(const ExtAssetDefinition& rhs) const
{
	return GetUniqueId() == rhs.GetUniqueId();
}

bool
ExtAssetDefinition::operator!=(const ExtAssetDefinition& rhs) const
{
	return !(*this == rhs);
}

bool
ExtAssetDefinition::operator<(const ExtAssetDefinition& rhs) const
{
	return GetUniqueId() < rhs.GetUniqueId();
}

const std::string&
ExtAssetDefinition::GetName() const &
{
	return _name;
}

const std::string&
ExtAssetDefinition::GetVersion() const &
{
	return _version;
}

const pxr::SdfAssetPath&
ExtAssetDefinition::GetIdentifier() const &
{
	return _identifier;
}

const std::string&
ExtAssetDefinition::GetUniqueId() const &
{
	return std::visit(
			[](const auto& id) -> const std::string& {
				return id.value;
			}, _uniqueId);
}

bool
ExtAssetDefinition::HasMetadata() const
{
	return _metadata != std::nullopt;
}

bool
ExtAssetDefinition::HasCustomUniqueId() const
{
	return std::holds_alternative<_CustomUniqueId>(_uniqueId);
}

const std::optional<pxr::VtDictionary>&
ExtAssetDefinition::GetMetadata() const &
{
	return _metadata;
}

bool
ExtAssetDefinition::IsValid(std::string* reason) const
{
	// TODO: figure out what characters are valid/invalid in the generated (or user
	// supplied) UID. It's likely these will be derived from URI identifiers and
	// so we can't be overly restrictive. For now disallow the uid if it doesn't
	// begin with an ascii alpha character or an underscore (matching USD/SDF
	// prim-name rules). MakeDefaultUniqueId sanitizes default uids to satisfy
	// this, but custom uids supplied via the customUniqueId constructor still
	// have to meet it on their own.
	const std::string& uid = GetUniqueId();

	if (!_StartsWithValidIdChar(uid))
	{
		if (reason)
		{
			*reason = "unique identifier must start with an ascii alpha "
			          "character or an underscore";
		}
		return false;
	}

	if(!_name.empty()
		&& !_identifier.GetAuthoredPath().empty()
		&& !_identifier.GetResolvedPath().empty())
	{
		return true;
	}

	if (reason)
	{
		std::string result;

		if (_name.empty())
		{
			result += "asset name cannot be empty";
		}

		if (_identifier.GetAuthoredPath().empty())
		{
			if (!result.empty())
			{
				result += ", ";
			}
			result += "asset identifier cannot have an empty authored path";
		}

		if (_identifier.GetResolvedPath().empty())
		{
			if (!result.empty())
			{
				result += ", ";
			}
			result += "asset identifier cannot have an empty resolved path";
		}

		result += ".";
		*reason = std::move(result);
	}

	return false;
}

ExtAssetDefinition::operator bool() const noexcept
{
	return IsValid();
}

// static
std::string
ExtAssetDefinition::MakeDefaultUniqueId(
	const std::string& name,
	const std::string& version)
{
	std::string uid = version.empty()
		? name
		: name + "_v" + version;

	// Sanitize the leading character so the resulting uid satisfies IsValid()
	if (!_StartsWithValidIdChar(uid))
	{
		uid.insert(uid.begin(), '_');
	}

	return uid;
}

// static
bool
ExtAssetDefinition::HasSameFields(
	const ExtAssetDefinition& defn1,
	const ExtAssetDefinition& defn2)
{
	// empty metadata dict and unset optional compare equal
	auto _MetadataIsSame = [&defn1, &defn2]() -> bool {
		static const pxr::VtDictionary empty;
		const std::optional<pxr::VtDictionary>& a = defn1.GetMetadata();
		const std::optional<pxr::VtDictionary>& b = defn2.GetMetadata();
		const pxr::VtDictionary& da = a ? *a : empty;
		const pxr::VtDictionary& db = b ? *b : empty;
		return da == db;
	};

	auto _IdentifierIsSame = [&defn1, &defn2]() -> bool
	{
		// Only compare the authored path, not the evaluated or resolved paths,
		// since an asset resolver may be intentionally pointing at files in
		// different filesystem locations.
		return
			defn1.GetIdentifier().GetAuthoredPath()
			== defn2.GetIdentifier().GetAuthoredPath();
	};

	return
		   defn1.GetName() == defn2.GetName()
		&& defn1.GetVersion() == defn2.GetVersion()
		&& _IdentifierIsSame()
		&& defn1.GetUniqueId() == defn2.GetUniqueId()
		&& _MetadataIsSame();
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK

