// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "item.h"

#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/primPermutation.h"
#include "UsdPregen/types.h"

#include "util.h"

#include "USDIncludesStart.h"
#include "pxr/base/arch/env.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/span.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/sdf/primSpec.h"
#include "pxr/usd/sdf/spec.h"
#include "pxr/usd/sdf/variantSpec.h"
#include "pxr/usd/usd/prim.h"
#include "USDIncludesEnd.h"

#include <algorithm>
#include <memory>
#include <optional>
#include <string>
#include <vector>


#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE

PREGEN_NAMESPACE_OPEN_SCOPE

static bool _ContentPathWritesAreEnabled()
{
	static const bool enabled = []()
	{
		const std::string value = pxr::ArchGetEnv("USDPREGEN_STORE_CONTENT_PATHS");
		if (value.empty())
		{
			return true; // default = "1"
		}

		return value != "0"; // anything except "0" = true
	}();

	return enabled;
}

namespace internal
{

ItemScope::ItemScope(
	const ScopeHandle& handle,
	const pxr::SdfPath& path,
	const ItemScope* parent,
	const Item* owner)
	: _handle(handle)
	, _path(path)
	, _parent(parent)
	, _owner(owner)
{
	TF_AXIOM(_owner);

	_opinionDepth = _owner->GetDepth();

	if (const Item* itemParent = _owner->GetParentItem())
	{
		const ItemScope* parentScope = itemParent->GetCurrentScope();
		TF_AXIOM(parentScope);
		_consumerDepth = parentScope->_consumerDepth;
	}

	DEBUG_ITEM(
		"Constructed new item scope ...\n"
		".. owning asset uid: %s\n"
		".. owning path: %s\n"
		".. scope path: %s\n"
		".. consumer depth: %d\n"
		, _owner->GetUniqueId().c_str()
		, _owner->GetPath().GetText()
		, _path.GetText()
		, _consumerDepth
	);
}

ItemScope::ItemScope(
	const ScopeHandle& handle,
	const pxr::SdfPath& path,
	const ItemScope* parent,
	const PrimPermutationRefPtr& perm,
	bool consumesDescendants)
	: _handle(handle)
	, _path(path)
	, _parent(parent)
	, _perm(perm)
{
	TF_AXIOM(_parent);

	_owner = _parent->_owner;
	_opinionDepth = _owner->GetDepth();

	const types::int32 parentConsumerDepth = _parent->_consumerDepth;
	if (consumesDescendants && parentConsumerDepth == types::InvalidIndex)
	{
		// start consuming dependents from the current owners depth
		_consumerDepth = _owner->GetDepth();
	}
	else
	{
		// otherwise inherit the parent scopes consumer depth
		_consumerDepth = parentConsumerDepth;
	}

	DEBUG_ITEM(
		"Constructed new permutation scope ...\n"
		".. owning asset uid: %s\n"
		".. owning path: %s\n"
		".. scope path: %s\n"
		".. consumes descendents: %d\n"
		".. consumer depth: %d\n"
		, _owner->GetUniqueId().c_str()
		, _owner->GetPath().GetText()
		, _path.GetText()
		, consumesDescendants
		, _consumerDepth
	);
}

const pxr::SdfPath&
ItemScope::GetPath() const &
{
	return _path;
}

ScopeHandle
ItemScope::GetHandle() const
{
	return _handle;
}

const ItemScope*
ItemScope::GetParent() const
{
	return _parent;
}

types::int32
ItemScope::GetConsumerDepth() const
{
	return _consumerDepth;
}

types::int32
ItemScope::GetOpinionDepth() const
{
	return _opinionDepth;
}

bool
ItemScope::AddContent(const pxr::TfToken& category, const pxr::SdfPath& path)
{
	TF_VERIFY(!category.IsEmpty());

	// Note an invalid category identifier will only be checked once. The caller
	// should therefore not repeatedly call with invalid category tokens.

	auto [itr, inserted] = _contentCountsByCategory.try_emplace(category, 0);
	if (inserted && !pxr::SdfPath::IsValidIdentifier(category))
	{
		return false;
	}

	// Increment the count for this category
	++itr->second;

	// Optionally store the path
	if (_ContentPathWritesAreEnabled())
	{
		_contentPathsByCategory[category].push_back(path);
	}

	return true;
}

bool
ItemScope::HasContent() const
{
	return !_contentCountsByCategory.empty();
}

pxr::TfTokenVector
ItemScope::GetContentCategoryNames() const
{
	pxr::TfTokenVector result;

	result.reserve(_contentCountsByCategory.size());
	for (const auto& [category, _] : _contentCountsByCategory)
	{
		result.push_back(category);
	}

	return result;
}

ItemScope::CategoryView
ItemScope::GetCategoryView(const pxr::TfToken& category) const
{
	// Counts must exist
	auto countItr = _contentCountsByCategory.find(category);
	if (!TF_VERIFY(countItr != _contentCountsByCategory.end()))
	{
		static const pxr::SdfPathVector emptyPaths;
		return { 0, emptyPaths };
	}

	// Paths are optional
	auto pathsItr = _contentPathsByCategory.find(category);
	if (pathsItr != _contentPathsByCategory.end())
	{
		return { countItr->second, pathsItr->second };
	}

	static const pxr::SdfPathVector emptyPaths;

	return { countItr->second, emptyPaths };
}

bool
ItemScope::AddOverridePathIfNeeded(const pxr::SdfPath& path, types::int32 depth)
{
	if (!TF_VERIFY(depth > types::InvalidIndex))
	{
		return false;
	}

	if (!TF_VERIFY(path.IsPropertyPath()))
	{
		return false;
	}

	if (depth < _opinionDepth)
	{
		DEBUG_ITEM(
		    "Reducing opinion depth for scope <%s> from (%d) to (%d) "
			"due to override first encountered via property <%s>\n"
			, _path.GetText()
			, _opinionDepth
			, depth
			, path.GetText()
	    );

		_opinionDepth = depth;
	}

	// If the override is coming from ancestor asset append it to the
	// relevant lists.
	if (depth < _owner->GetDepth())
	{
		_unencapsulatedPropertyPaths.push_back(path);
		_unencapsulatedPropertyDepths.push_back(depth);
		return true;
	}

	return false;
}

TfSpan<const pxr::SdfPath>
ItemScope::GetOverridePaths() const
{
	return TfMakeConstSpan(_unencapsulatedPropertyPaths);
}

bool
ItemScope::HasDependencies() const
{
	return !_dependsOn.empty();
}

bool
ItemScope::AddDependency(const pxr::SdfPath& path)
{
	auto [itr, inserted] = _dependsOn.insert(path);
	return inserted;
}

const pxr::SdfPathSet&
ItemScope::GetDependencies() const &
{
	return _dependsOn;
}

bool
ItemScope::AddDescendantDefinitionScenePath(
	const pxr::SdfPath& path,
	const bool isEncapsulated)
{
	if (TF_VERIFY(path.IsPrimPath()))
	{
		pxr::SdfPathSet& pathSet = isEncapsulated
			? _encapsulatedDefinitionScenePaths
			: _unencapsulatedDefinitionScenePaths;

		auto [_, inserted] = pathSet.insert(path);
		return inserted;
	}

	return false;
}

const pxr::SdfPathSet&
ItemScope::GetEncapsulatedDefinitionPaths() const &
{
	return _encapsulatedDefinitionScenePaths;
}

const pxr::SdfPathSet&
ItemScope::GetUnencapsulatedDefinitionPaths() const &
{
	return _unencapsulatedDefinitionScenePaths;
}

std::shared_ptr<const PrimPermutation>
ItemScope::GetPermutation() const
{
	return _perm;
}

Item::Item(
	const pxr::UsdPrim& prim,
	const ExtAssetDefinition* assetDefn,
	const ScopeHandle& handle,
	Item* parent,
	const std::optional<pxr::SdfPathSet>& candidateOverridePaths)
	: _path(prim.GetPath())
	, _parent(parent)
{
	TF_AXIOM(assetDefn);

	const std::string& uid = assetDefn->GetUniqueId();
	if (_parent)
	{
		_depth = 1 + _parent->GetDepth();

		DEBUG_ITEM(
			"Constructed new item ...\n"
			".. asset uid: %s\n"
			".. scene path: %s\n"
			".. parent uid: %s\n"
			".. parent consumer depth: %d\n"
			, uid.c_str()
			, _path.GetText()
			, _parent->GetUniqueId().c_str()
			, _parent->GetCurrentScope()->GetConsumerDepth()
		);
	}
	else
	{
		DEBUG_ITEM(
			"Constructed root item ...\n"
			".. asset uid: %s\n"
			".. scene path: %s\n"
			, uid.c_str()
			, _path.GetText()
		);
	}

	UpdateAssetDefinition(prim, assetDefn, handle, candidateOverridePaths);
}

const pxr::SdfPath&
Item::GetPath() const &
{
	return _path;
}

const ExtAssetDefinition*
Item::GetDefinition() const
{
	return _ctx->assetDefn;
}

const std::string&
Item::GetUniqueId() const &
{
	return _ctx->assetDefn->GetUniqueId();
}

types::int32
Item::GetDepth() const
{
	return _depth;
}

const pxr::SdfAssetPath&
Item::GetIdentifier() const &
{
	return _ctx->assetDefn->GetIdentifier();
}

const TfToken&
Item::GetDefaultPrim() const &
{
	return _ctx->defaultPrim;
}

const pxr::SdfPath&
Item::GetPathAtIntroduction() const &
{
	return _ctx->pathAtIntroduction;
}

Item*
Item::GetParentItem()
{
	return _parent;
}

const Item*
Item::GetParentItem() const
{
	return _parent;
}

bool
Item::ContainsPermutations() const
{
	return _scopes->size() > 1;
}

types::int32
Item::GetOverrideDepth() const
{
	TF_AXIOM(!_scopes->empty());
	return _scopes->back()->GetOpinionDepth();
}

ItemScope*
Item::PushScope(
	const ScopeHandle& handle,
	const pxr::SdfPath& path,
	const ItemScope* parent,
	const PrimPermutationRefPtr& perm,
	bool consumesDescendants)
{
	_scopes->push_back(
	  std::make_unique<ItemScope>(handle, path, parent, perm, consumesDescendants)
	);

	return _scopes->back().get();
}

void
Item::PopScope()
{
	TF_AXIOM(!_scopes->empty());
	_scopes->pop_back();
}

ItemScope*
Item::GetCurrentScope()
{
	return _scopes->empty() ? nullptr : _scopes->back().get();
}

const ItemScope*
Item::GetCurrentScope() const
{
	return _scopes->empty() ? nullptr : _scopes->back().get();
}

ItemScope*
Item::GetPrimScope()
{
	return _scopes->empty() ? nullptr : _scopes->at(0).get();
}

const ItemScope*
Item::GetPrimScope() const
{
	return _scopes->empty() ? nullptr : _scopes->at(0).get();
}

bool
Item::IsRootItem() const
{
	return _parent == nullptr;
}

size_t
Item::NumScopes() const
{
	return _scopes->size();
}

ItemScope*
Item::GetScope(size_t idx)
{
	if (TF_VERIFY(idx < _scopes->size()))
	{
		return _scopes->at(idx).get();
	}
	return nullptr;
}

const ItemScope*
Item::GetScope(size_t idx) const
{
	if (TF_VERIFY(idx < _scopes->size()))
	{
		return _scopes->at(idx).get();
	}
	return nullptr;
}

void
Item::UpdateAssetDefinition(
	const pxr::UsdPrim& prim,
	const ExtAssetDefinition* assetDefn,
	const ScopeHandle& handle,
	const std::optional<pxr::SdfPathSet>& candidateOverridePaths)
{
	TF_AXIOM(assetDefn);

	_ctx = std::make_unique<_Context>();
	_ctx->assetDefn = assetDefn;
	_ctx->candidateOverridePaths = candidateOverridePaths;

	util::GetIntroPathAndDefaultPrim(
		prim,
		assetDefn->GetIdentifier(),
		&_ctx->pathAtIntroduction,
		&_ctx->defaultPrim);

	_scopes = &_ctx->scopes;
	_scopes->push_back(
		std::make_unique<ItemScope>(handle, _path, nullptr, this)
	);
}

bool
Item::ShouldCollectOverridesForPrim(const pxr::SdfPath& path) const
{
	const std::optional<pxr::SdfPathSet>& paths = _ctx->candidateOverridePaths;

	// Unset optional means consider all paths
	if (!paths)
	{
		return true;
	}

	// Empty path set means skip all overrides for the current item
	if (paths->empty())
	{
		return false;
	}

	return static_cast<bool>(paths->count(path));
}

} // namespace internal

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
