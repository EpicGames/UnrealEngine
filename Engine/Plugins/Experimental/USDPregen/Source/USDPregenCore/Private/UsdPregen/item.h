// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include <cstdint>  // std::uint32_t
#include <cstddef>  // std::size_t
#include <memory>
#include <set>
#include <string>
#include <vector>

#include "UsdPregen/types.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/hash.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/sdf/spec.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE

	class SdfPrimSpec;
	class SdfVariantSpec;
	class UsdPrim;

	template <typename T>
	class TfSpan;

	template <typename T>
	class SdfHandle;

	using SdfPrimSpecHandle = SdfHandle<SdfPrimSpec>;
	using SdfVariantSpecHandle = SdfHandle<SdfVariantSpec>;

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

class ExtAssetDefinition;
using PrimPermutationRefPtr = std::shared_ptr<class PrimPermutation>;
using PrimPermutationConstRefPtr = std::shared_ptr<const class PrimPermutation>;

namespace internal
{

struct ScopeHandle
{
	pxr::SdfPath id;
};

class Item;

/// \class ItemScope
///
/// Represents a region of scene description belonging to a particular
/// external asset instance.
///
/// Each ItemScope corresponds to a specific composition context within
/// the USD stage. Scopes are created for:
///
///	  * the base prim associated with an asset definition
///	  * permutations introduced by composition operations (variants,
///		inherits, etc.)
///
/// Permutations may introduce nested scopes, forming a stack of scopes
/// associated with a single Item. The SceneTracker is responsible for pushing
/// and popping scopes as traversal enters and exits permutation contexts.
///
/// An ItemScope records information discovered while traversing the
/// scene beneath that scope, including:
///
///	  * categorized counts and paths that produce content
///	  * property paths that override and modify a descendant asset's state
///	  * dependency relationships between scopes
///
/// Scope depth values are used to determine how opinions from ancestor
/// assets interact with the current asset's scene description.
///
/// Instances of ItemScope are owned by Item objects.
class ItemScope
{
public:

	/// Construct the initial (root) scope for an Item.
	///
	/// This scope corresponds to the prim that introduced the asset
	/// definition and represents the non-permutation scene description.
	ItemScope(const ScopeHandle& handle,
			  const pxr::SdfPath& path,
			  const ItemScope* parent,
			  const Item* owner);

	/// Construct a permutation scope.
	///
	/// Permutation scopes represent additional composition contexts
	/// created by permutation operations (variants, inherits, etc.).
	ItemScope(const ScopeHandle& handle,
			  const pxr::SdfPath& path,
			  const ItemScope* parent,
			  const PrimPermutationRefPtr& perm,
		      bool consumesDescendants);

	/// View on the internal map state of a specific content category.
	struct CategoryView
	{
		std::int32_t count = 0;
		const pxr::SdfPathVector& paths;
	};

	/// Returns the scene path associated with this scope.
	const pxr::SdfPath& GetPath() const &;

	/// Returns the Sdf spec handle representing the scope.
	ScopeHandle GetHandle() const;

	/// Returns the parent scope.
	const ItemScope* GetParent() const;

	/// Returns the depth of the ancestor asset that consumes descendant
	/// targets originating in this scope.
	types::int32 GetConsumerDepth() const;

	/// Returns the minimum depth at which overriding opinions were found.
	types::int32 GetOpinionDepth() const;

	/// Add content for the given category and path.
	bool AddContent(const pxr::TfToken& category, const pxr::SdfPath& path);

	/// Returns true if this scope holds content or false if the
	/// scope is effectively a grouping of other scopes.
	bool HasContent() const;

	/// Get all the content category names registered on this scope.
	pxr::TfTokenVector GetContentCategoryNames() const;

	/// Get the count and (optional) paths for the given category.
	CategoryView GetCategoryView(const pxr::TfToken& category) const;

	/// Add the given property as an override path - if the property at the
	/// given depth is less than the scopes depth, add the property path to the
	// list of overrides for this scope.
	bool AddOverridePathIfNeeded(const pxr::SdfPath& path, types::int32 depth);

	/// Returns paths whose opinions override ancestor assets.
	pxr::TfSpan<const pxr::SdfPath> GetOverridePaths() const;

	/// Returns true if this scope has one or more dependencies.
	bool HasDependencies() const;

	/// Add a dependency to this scope.
	bool AddDependency(const pxr::SdfPath& path);

	/// Get all dependencies for this scope.
	const pxr::SdfPathSet& GetDependencies() const &;

	/// Adds an encapsulated/unencapsulated descendant scene path to the current
	/// scope. The scene path provided should represent the location of an asset
	/// definition.
	bool AddDescendantDefinitionScenePath(const pxr::SdfPath& path,
										  bool isEncapsulated);

	// Get the encapsulated definition paths.
	const pxr::SdfPathSet& GetEncapsulatedDefinitionPaths() const &;

	// Get the unencapsulated definition paths.
	const pxr::SdfPathSet& GetUnencapsulatedDefinitionPaths() const &;

	/// Returns the permutation associated with this scope, if any.
	PrimPermutationConstRefPtr GetPermutation() const;

private:

	// The spec representing this scope, either an SdfPrimSpec or
	// SdfVariantSpec.
	const ScopeHandle _handle;

	// Scene path associated with this scope.
	const pxr::SdfPath _path;

	// Parent scope (null only for the root scope of an Item).
	const ItemScope* _parent = nullptr;

	// The Item that owns this scope.
	const Item* _owner = nullptr;

	// Depth of the strongest opinion encountered within this scope.
	types::int32 _opinionDepth = types::InvalidIndex;

	// Depth of the ancestor asset that consumes descendant targets.
	types::int32 _consumerDepth = types::InvalidIndex;

	// Permutation associated with this scope, if any.
	const PrimPermutationConstRefPtr _perm = nullptr;

	// The property paths and depths of the overrides active on this scope.
	pxr::SdfPathVector _unencapsulatedPropertyPaths;
	std::vector<types::int32> _unencapsulatedPropertyDepths;

	// The set of resolved targets this item depends on.
	pxr::SdfPathSet _dependsOn;

	// The scene paths of the current scopes descendant definitions. This can be
	// useful in order to quickly acquire the locations of nested assets. The
	// paths are separated depending on whether or not the asset is fully
	// encapsulated (has no overrides) or is unencapsulated (requires the opinions
	// of one or more ancestor assets to faithfully reproduce its current state).
	pxr::SdfPathSet _encapsulatedDefinitionScenePaths;
	pxr::SdfPathSet _unencapsulatedDefinitionScenePaths;

	// Content counts and paths. These are separated into two maps because the
	// path storage can be expensive on large scenes. The counts are mandatory
	// however the path storage can be disabled by setting environment variable:
	//
	// USDPREGEN_STORE_CONTENT_PATHS=0
	//
	std::unordered_map<pxr::TfToken,
					   std::int32_t,
					   pxr::TfHash>  _contentCountsByCategory;

	std::unordered_map<pxr::TfToken,
					   pxr::SdfPathVector,
					   pxr::TfHash>  _contentPathsByCategory;

	// A list of prim paths provided by the user that controls the prims inside
	// the asset that can contribute override options, and thus influence the
	// eventual creation of a target. Supplying a specific subset of paths (if
	// known) can accelerate the tracker traversal. If can also be used as a
	// broad filter to prevent overrides getting picked up accidentally.
	std::optional<pxr::SdfPathSet> _candidateOverridePaths;
};


/// \class Item
///
/// Internal container representing a single external asset instance
/// encountered during traversal.
///
/// An Item corresponds to a USD prim that introduced an
/// ExtAssetDefinition. Items form a hierarchy mirroring the nested
/// asset structure discovered during stage traversal.
///
/// Each Item owns a stack of ItemScope objects. The first scope always
/// represents the base prim scope. Additional scopes may be pushed when
/// permutations are encountered during traversal.
///
/// The SceneTracker manages the lifetime and ordering of scopes by pushing
/// and popping them as the traversal enters and exits permutation
/// contexts.
///
/// Items may also switch contexts when a permutation composes a different
/// external asset definition. In this case UpdateAssetDefinition() is
/// invoked to replace the active context while preserving traversal state.
///
/// Item depth corresponds to the nesting level of asset definitions in
/// the stage and is used to reason about override and dependency behavior.
class Item
{
public:

	Item(const pxr::UsdPrim& prim,
		 const ExtAssetDefinition* assetDefn,
		 const ScopeHandle& handle,
		 Item* parent,
		 const std::optional<pxr::SdfPathSet>& candidateOverridePaths);

	/// Returns the scene path for the prim that introduced this item.
	const pxr::SdfPath& GetPath() const &;

	/// Returns the asset definition for this item.
	const ExtAssetDefinition* GetDefinition() const;

	/// Returns the unique identifier of the active asset definition.
	const std::string& GetUniqueId() const &;

	/// Returns the depth of this item within the asset hierarchy.
	types::int32 GetDepth() const;

	const pxr::SdfAssetPath& GetIdentifier() const &;

	const pxr::TfToken& GetDefaultPrim() const &;

	const pxr::SdfPath& GetPathAtIntroduction() const &;

	bool ContainsPermutations() const;

	types::int32 GetOverrideDepth() const;

	ItemScope* PushScope(const ScopeHandle& handle,
						 const pxr::SdfPath& path,
					     const ItemScope* parent,
						 const PrimPermutationRefPtr& perm,
						 bool consumesDescendants);

	void PopScope();

	ItemScope* GetCurrentScope();

	const ItemScope* GetCurrentScope() const;

	Item* GetParentItem();

	const Item* GetParentItem() const;

	ItemScope* GetPrimScope();

	const ItemScope* GetPrimScope() const;

	bool IsRootItem() const;

	std::size_t NumScopes() const;

	ItemScope* GetScope(size_t idx);

	const ItemScope* GetScope(size_t idx) const;

	/// Replace the active asset definition context.
	void UpdateAssetDefinition(
	         const pxr::UsdPrim& prim,
		     const ExtAssetDefinition* assetDefn,
		     const ScopeHandle& handle,
		     const std::optional<pxr::SdfPathSet>& candidateOverridePaths);

	/// Returns true if the given path should be inspected for overrides.
	bool ShouldCollectOverridesForPrim(const pxr::SdfPath& path) const;

private:

	using ItemScopeList = std::vector<std::unique_ptr<ItemScope>>;

	// The current context of this item - the asset definition, intro path,
	// default prim, scope list, and allowed override paths.
	//
	// Variants, and composition changes in general, may swap the underlying
	// external asset an item (location in namespace) reflects. Rather than
	// attempt to construct a new in these cases, we instead update the
	// context entry via UpdateAssetDefinition()
	struct _Context
	{
		const ExtAssetDefinition* assetDefn;
		pxr::SdfPath pathAtIntroduction;
		pxr::TfToken defaultPrim;
		ItemScopeList scopes;
		std::optional<pxr::SdfPathSet> candidateOverridePaths;
	};

	// The scene path of this location in namespace. This path will always be
	// a regular prim path (in contrast to item scope paths which may include
	// variant selections.
	const pxr::SdfPath _path;

	// The depth of the asset this item represents
	types::int32 _depth = 0;

	/// Parent item in the asset hierarchy.
	Item* _parent = nullptr;

	/// Active context for the current asset definition.
	std::unique_ptr<_Context> _ctx = nullptr;

	/// Pointer to the current scope list for convenience (owned by the context).
	ItemScopeList* _scopes = nullptr;
};

} // namespace internal

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
