// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/jsonManifestSerializer.h"

#include "UsdPregen/assetDefinitionRegistry.h"
#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/manifest.h"
#include "UsdPregen/permutationOps.h"
#include "UsdPregen/target.h"

#include "util.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/tf/staticTokens.h"
#include "pxr/base/tf/token.h"
#include "pxr/base/js/json.h"
#include "pxr/base/js/value.h"
#include "pxr/base/vt/dictionary.h"
#include "pxr/usd/sdf/assetPath.h"
#include "pxr/usd/sdf/path.h"
#include "USDIncludesEnd.h"

#include <cstddef>
#include <functional>
#include <limits>
#include <optional>
#include <set>
#include <sstream>
#include <string>
#include <unordered_map>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE

// clang-format off
TF_DEFINE_PRIVATE_TOKENS(
	_tokens,
	((jsonEncoding, "application/json"))
	((jsonSuffix, ".json"))
	((schemaName, "ue.usdpregen.manifest"))
);
// clang-format on

PREGEN_NAMESPACE_OPEN_SCOPE

using namespace internal::util;

namespace
{

constexpr std::int32_t _schemaVersion = 1;

constexpr std::size_t _maxFileSizeBytes = 64 * 1024 * 1024; // 64 MiB

class _ValueExtractor
{
public:

	std::optional<std::reference_wrapper<const pxr::JsObject>>
	GetObject(const pxr::JsObject& obj, const char* key, bool emptyOk = false) const
	{
		auto it = obj.find(key);
		if (it == obj.end() || !it->second.IsObject())
		{
			return std::nullopt;
		}

		const pxr::JsObject& value = it->second.GetJsObject();
		if (value.empty() && !emptyOk)
		{
			return std::nullopt;
		}

		return std::cref(value);
	}

	std::optional<std::string>
	GetString(const pxr::JsObject& obj, const char* key, bool emptyOk = false) const
	{
		auto it = obj.find(key);
		if (it == obj.end() || !it->second.IsString())
		{
			return std::nullopt;
		}

		const std::string value = it->second.GetString();
		if (value.empty() && !emptyOk)
		{
			return std::nullopt;
		}

		return value;
	}

	std::optional<std::int32_t>
	GetInt(const pxr::JsObject& obj, const char* key) const
	{
		auto it = obj.find(key);
		if (it == obj.end())
		{
			return std::nullopt;
		}

		const pxr::JsValue& value = it->second;

		if (value.IsInt())
		{
			return value.GetInt();
		}

		if (value.IsUInt64())
		{
			const std::uint64_t v = value.GetUInt64();
			if (v <= static_cast<std::uint64_t>(std::numeric_limits<std::int32_t>::max()))
			{
				return static_cast<std::int32_t>(v);
			}

			return std::nullopt;
		}

		if (value.Is<std::int64_t>()) // no IsInt64() API
		{
			const std::int64_t v = value.GetInt64();
			if (v >= std::numeric_limits<std::int32_t>::min() &&
				v <= std::numeric_limits<std::int32_t>::max())
			{
				return static_cast<std::int32_t>(v);
			}
		}

		return std::nullopt;
	}

	std::optional<std::reference_wrapper<const pxr::JsArray>>
	GetArray(const pxr::JsObject& obj, const char* key, bool emptyOk = false) const
	{
		auto it = obj.find(key);
		if (it == obj.end() || !it->second.IsArray())
		{
			return std::nullopt;
		}

		const pxr::JsArray& value = it->second.GetJsArray();
		if (value.empty() && !emptyOk)
		{
			return std::nullopt;
		}

		return std::cref(value);
	}

	std::optional<std::reference_wrapper<const pxr::JsObject>>
	RequireObject(const pxr::JsObject& obj, const char* key, bool emptyOk = false) const
	{
		auto value = GetObject(obj, key, emptyOk);
		if (!value)
		{
			_Warn("expected object", key, emptyOk);
		}
		return value;
	}

	std::optional<std::string>
	RequireString(const pxr::JsObject& obj, const char* key, bool emptyOk = false) const
	{
		auto value = GetString(obj, key, emptyOk);
		if (!value)
		{
			_Warn("expected string", key, emptyOk);
		}
		return value;
	}

	std::optional<std::int32_t>
	RequireInt(const pxr::JsObject& obj, const char* key) const
	{
		auto value = GetInt(obj, key);
		if (!value)
		{
			_Warn("expected int", key, true);
		}
		return value;
	}

	std::optional<std::reference_wrapper<const pxr::JsArray>>
	RequireArray(const pxr::JsObject& obj, const char* key, bool emptyOk = false) const
	{
		auto value = GetArray(obj, key, emptyOk);
		if (!value)
		{
			_Warn("expected array", key, emptyOk);
		}
		return value;
	}

private:
	void _Warn(const char* expectedType, const char* key, bool emptyOk) const
	{
		TF_WARN(
			"Failed to parse buffer - %s%s for key: %s"
			, expectedType
			, emptyOk ? "" : "non-empty "
			,key
		);
	}
};

TargetUid
_ExtractTargetUid(const _ValueExtractor& extract, const pxr::JsObject& root)
{
	const auto target = extract.RequireObject(root, "target");
	if (!target) return {};

	const auto uid = extract.RequireObject(target->get(), "uid");
	if (!uid) return {};

	const auto defnUid = extract.RequireString(uid->get(), "definition_uid");
	if (!defnUid) return {};

	// permutation uid may be empty
	const auto permUid = extract.RequireString(uid->get(), "permutation_uid", true);
	if (!permUid) return {};

	return TargetUid{ *defnUid, *permUid };
}

std::optional<Product>
_ExtractProduct(const _ValueExtractor& extract, const pxr::JsObject& obj)
{
	Product product;

	const auto upackagePath = extract.RequireString(obj, "upackage_path");
	if (!upackagePath) return std::nullopt;
	product.upackagePath = *upackagePath;

	const auto uclass = extract.RequireString(obj, "uclass");
	if (!uclass) return std::nullopt;
	product.uclass = *uclass;

	const auto unodeId = extract.RequireString(obj, "unode_id");
	if (!unodeId) return std::nullopt;
	product.unodeId = *unodeId;

	// prim type may be empty
	const auto usdPrimType = extract.RequireString(obj, "usd_prim_type", true);
	if (!usdPrimType) return std::nullopt;
	product.usdPrimType = *usdPrimType;

	const auto usdPrimPath = extract.RequireString(obj, "usd_prim_path", true);
	if (!usdPrimPath) return std::nullopt;
	product.usdPrimPath = *usdPrimPath;

	return product;
}

// ---------------------------------------------------------------------- //
// PermutationOp <-> JSON
// ---------------------------------------------------------------------- //

pxr::JsObject
_OpToJs(const PermutationOpRefPtr& op)
{
	pxr::JsObject obj;

	const SerializedOp serialized = ToSerializedOp(op);
	obj["type_name"] = pxr::JsValue(serialized.typeName);

	pxr::JsArray args;
	for (const SerializedOpArg& arg : serialized.args)
	{
		pxr::JsObject argObj;
		argObj["name"] = pxr::JsValue(arg.name);
		argObj["type"] = pxr::JsValue(arg.typeName);
		argObj["value"] = pxr::JsValue(arg.value);
		args.push_back(argObj);
	}
	obj["opargs"] = args;

	return obj;
}

PermutationOpRefPtr
_OpFromJs(const _ValueExtractor& extract, const pxr::JsObject& obj)
{
	const auto typeName = extract.RequireString(obj, "type_name");
	if (!typeName) return nullptr;

	SerializedOp serialized;
	serialized.typeName = *typeName;

	if (const auto args = extract.GetArray(obj, "opargs", /*emptyOk*/true))
	{
		for (const pxr::JsValue& argVal : args->get())
		{
			if (!argVal.IsObject())
			{
				continue;
			}
			const pxr::JsObject& argObj = argVal.GetJsObject();

			const auto argName = extract.RequireString(argObj, "name");
			if (!argName) continue;

			const auto argType = extract.RequireString(argObj, "type");
			if (!argType) continue;

			const auto argValue = extract.RequireString(argObj, "value", /*emptyOk*/true);
			if (!argValue) continue;

			SerializedOpArg arg;
			arg.name = *argName;
			arg.typeName = *argType;
			arg.value = *argValue;
			serialized.args.push_back(std::move(arg));
		}
	}

	return FromSerializedOp(serialized);
}

// ---------------------------------------------------------------------- //
// ExtAssetDefinition <-> JSON
// ---------------------------------------------------------------------- //

// Each definition entry carries the scene path and permutation ops contributed
// by the matching TargetDefinitionEntry from the originating TargetData stack. This
// folds the TargetDefinitionEntry data onto its definition (1:1, deduped by UniqueId)
// rather than emitting a parallel infos array.
pxr::JsObject
_DefinitionToJs(const ExtAssetDefinition& defn,
                const TargetDefinitionEntry& info)
{
	pxr::JsObject obj;

	obj["unique_id"] = pxr::JsValue(defn.GetUniqueId());
	obj["name"] = pxr::JsValue(defn.GetName());
	obj["version"] = pxr::JsValue(defn.GetVersion());

	pxr::JsObject identifier;
	identifier["authored"] = pxr::JsValue(defn.GetIdentifier().GetAuthoredPath());
	identifier["resolved"] = pxr::JsValue(defn.GetIdentifier().GetResolvedPath());
	obj["identifier"] = identifier;

	obj["has_custom_unique_id"] = pxr::JsValue(defn.HasCustomUniqueId());

	std::string metadataBlob;
	if (const std::optional<pxr::VtDictionary>& metadata = defn.GetMetadata())
	{
		metadataBlob = MetadataToUsdaBlob(*metadata);
	}
	obj["metadata"] = pxr::JsValue(metadataBlob);

	obj["scene_path"] = pxr::JsValue(info.GetScenePath().GetString());

	pxr::JsArray ops;
	for (const PermutationOpRefPtr& op : info.GetPermutationOps())
	{
		ops.push_back(_OpToJs(op));
	}
	obj["permutation_ops"] = ops;

	return obj;
}

// Decoded form of a definition entry: the rehydrated ExtAssetDefinition
// plus the scene path and permutation ops needed to rebuild the TargetDefinitionEntry
// stack on the deserialize side.
struct _DecodedDefinition
{
	ExtAssetDefinition defn;
	pxr::SdfPath scenePath;
	PermutationOpVector permutationOps;
};

std::optional<_DecodedDefinition>
_DefinitionFromJs(const _ValueExtractor& extract, const pxr::JsObject& obj)
{
	const auto uniqueId = extract.RequireString(obj, "unique_id");
	if (!uniqueId) return std::nullopt;

	const auto name = extract.RequireString(obj, "name", /*emptyOk*/true);
	if (!name) return std::nullopt;

	const auto version = extract.RequireString(obj, "version", /*emptyOk*/true);
	if (!version) return std::nullopt;

	const auto identifierObj = extract.RequireObject(obj, "identifier",
	                                                 /*emptyOk*/true);
	if (!identifierObj) return std::nullopt;

	const auto authored = extract.RequireString(identifierObj->get(), "authored",
	                                            /*emptyOk*/true);
	const auto resolved = extract.RequireString(identifierObj->get(), "resolved",
	                                            /*emptyOk*/true);
	if (!authored || !resolved) return std::nullopt;

	const pxr::SdfAssetPath identifier{*authored, *resolved};

	const auto metadataBlob = extract.RequireString(obj, "metadata",
	                                                /*emptyOk*/true);
	if (!metadataBlob) return std::nullopt;

	const std::optional<pxr::VtDictionary> metadata = UsdaBlobToMetadata(*metadataBlob);

	bool hasCustomUniqueId = false;
	if (auto it = obj.find("has_custom_unique_id");
		it != obj.end() && it->second.IsBool())
	{
		hasCustomUniqueId = it->second.GetBool();
	}

	const auto scenePathStr = extract.RequireString(obj, "scene_path");
	if (!scenePathStr) return std::nullopt;
	if (!pxr::SdfPath::IsValidPathString(*scenePathStr))
	{
		TF_WARN("Invalid scene path string: %s", scenePathStr->c_str());
		return std::nullopt;
	}

	PermutationOpVector ops;
	if (const auto opsArr = extract.GetArray(obj, "permutation_ops", /*emptyOk*/true))
	{
		for (const pxr::JsValue& opVal : opsArr->get())
		{
			if (!opVal.IsObject())
			{
				continue;
			}
			if (PermutationOpRefPtr op = _OpFromJs(extract, opVal.GetJsObject()))
			{
				ops.push_back(std::move(op));
			}
		}
	}

	// Pick the constructor variant that preserves HasCustomUniqueId() across
	// the round-trip. With a custom UID the value must be passed verbatim;
	// without, the no-customUniqueId form lets ExtAssetDefinition regenerate
	// the same UID from name + version and report HasCustomUniqueId() == false.
	auto buildDefn = [&]() -> ExtAssetDefinition {
		if (hasCustomUniqueId)
		{
			return metadata
				? ExtAssetDefinition(*name, *version, identifier, *uniqueId, *metadata)
				: ExtAssetDefinition(*name, *version, identifier, *uniqueId);
		}
		return metadata
			? ExtAssetDefinition(*name, *version, identifier, *metadata)
			: ExtAssetDefinition(*name, *version, identifier);
	};

	_DecodedDefinition result {
		buildDefn(),
		pxr::SdfPath(*scenePathStr),
		std::move(ops)
	};
	return result;
}

// ---------------------------------------------------------------------- //
// Dependencies
// ---------------------------------------------------------------------- //

pxr::JsArray
_TargetUidsToJs(const std::vector<TargetUid>& uids)
{
	pxr::JsArray result;
	for (const TargetUid& uid : uids)
	{
		pxr::JsObject obj;
		obj["definition_uid"] = pxr::JsValue(uid.GetDefinitionUid());
		obj["permutation_uid"] = pxr::JsValue(uid.GetPermutationUid());
		result.push_back(obj);
	}
	return result;
}

std::vector<TargetUid>
_TargetUidsFromJs(const _ValueExtractor& extract, const pxr::JsArray& arr)
{
	std::vector<TargetUid> result;
	result.reserve(arr.size());

	for (const pxr::JsValue& val : arr)
	{
		if (!val.IsObject())
		{
			continue;
		}
		const pxr::JsObject& obj = val.GetJsObject();

		const auto defnUid = extract.RequireString(obj, "definition_uid");
		if (!defnUid) continue;

		const auto permUid = extract.RequireString(obj, "permutation_uid",
		                                           /*emptyOk*/true);
		if (!permUid) continue;

		result.emplace_back(*defnUid, *permUid);
	}

	return result;
}

} // anonymous namespace

// virtual
bool
JsonManifestSerializer::Serialize(
	const Manifest& manifest,
	ManifestPayload& payload) const
{
	payload.encoding.clear();
	payload.data.clear();;

	if (!manifest.IsValid())
	{
		return false;
	}

	const TargetUid& targetUid = manifest.GetTargetUid();

	DEBUG_MANIFEST(
		"Begin manifest serialize for target (%s) with (%zu) products.\n"
		, pxr::TfStringify(targetUid).c_str()
		, manifest.GetProducts().size()
	);

	pxr::JsObject root;

	pxr::JsObject schema;

	schema["name"] = pxr::JsValue(JsonManifestSerializer::GetSchemaName());
	schema["version"] = pxr::JsValue(JsonManifestSerializer::GetSchemaVersion());

	root["schema"] = pxr::JsValue(schema);

	pxr::JsObject uid;
	uid["definition_uid"] = pxr::JsValue(targetUid.GetDefinitionUid());
	uid["permutation_uid"] = pxr::JsValue(targetUid.GetPermutationUid());

	pxr::JsObject target;
	target["uid"] = pxr::JsValue(uid);

	pxr::JsArray products;

	for (const Product& product : manifest.GetProducts())
	{
		pxr::JsObject p;

		p["upackage_path"] = pxr::JsValue(product.upackagePath);
		p["uclass"]		   = pxr::JsValue(product.uclass);
		p["unode_id"]	   = pxr::JsValue(product.unodeId);
		p["usd_prim_type"] = pxr::JsValue(product.usdPrimType);
		p["usd_prim_path"] = pxr::JsValue(product.usdPrimPath);

		products.push_back(p);
	}

	root["products"] = products;

	// IsValid() is checked at the top of Serialize, so target data is
	// guaranteed non-null and itself valid here.
	const TargetDataRefPtr& targetData = manifest.GetTargetData();

	// Dependencies live inside the "target" object alongside the uid.
	target["dependencies"] = _TargetUidsToJs(targetData->GetDependencies());

	// Top-level deduped definition snapshots. Each entry carries the scene
	// path and permutation ops contributed by the originating TargetDefinitionEntry
	// (folded onto the definition entry, 1:1, first occurrence wins). Order
	// follows the namespace stack walk.
	pxr::JsArray definitions;
	std::set<std::string> seen;
	for (const TargetDefinitionEntry& info : targetData->GetDefinitionEntries())
	{
		const ExtAssetDefinition* defn = info.GetDefinition();
		if (!defn) continue;
		if (!seen.insert(defn->GetUniqueId()).second) continue;

		definitions.push_back(_DefinitionToJs(*defn, info));
	}
	root["definitions"] = definitions;

	root["target"] = target;

	std::ostringstream stream;

	pxr::JsWriteToStream(JsValue(root), stream);

	const std::string str = stream.str();

	payload.encoding = _tokens->jsonEncoding.GetString();

	payload.data.assign(str.begin(), str.end());

	DEBUG_MANIFEST(
		"Serialize complete for target (%s), size: (%zu) bytes.\n"
		, pxr::TfStringify(targetUid).c_str()
		, payload.data.size()
	);

	return true;
}

// virtual
Manifest
JsonManifestSerializer::Deserialize(const ManifestPayload& payload) const
{
	if (payload.encoding != GetEncoding())
	{
		TF_WARN("Failed to deserialize manifest payload - unsupported encoding \"%s\""
				, payload.encoding.c_str());
		return {};
	}

	if (payload.data.empty())
	{
		TF_WARN("Failed to deserialize manifest payload - payload data is empty.");
		return {};
	}


	if (payload.data.size() > _maxFileSizeBytes)
	{
		TF_WARN("Failed to deserialize manifest payload - payload "
				"size is too large (greater than 64MiB)");
		return {};
	}

	const std::string jsonStr(reinterpret_cast<const char*>(payload.data.data()),
							  payload.data.size());
	pxr::JsParseError error;
	pxr::JsValue rootValue = pxr::JsParseString(jsonStr, &error);
	if (rootValue.IsNull())
	{
		TF_WARN("Failed to parse input (line %d, col %d) - %s",
			error.line, error.column, error.reason.c_str());
		return {};
	}

	if (!rootValue.IsObject())
	{
		TF_WARN("Invalid json document - root value is not an object");
		return {};
	}

	const pxr::JsObject& root = rootValue.GetJsObject();

	_ValueExtractor extract;

	const auto schema = extract.RequireObject(root, "schema");
	if (!schema) return {};

	const auto schemaName = extract.RequireString(schema->get(), "name");
	if (!schemaName) return {};

	if (*schemaName != std::string(JsonManifestSerializer::GetSchemaName()))
	{
		TF_WARN("Unexpected schema name (%s)", schemaName->c_str());
		return {};
	}

	const auto schemaVersion = extract.RequireInt(schema->get(), "version");
	if (!schemaVersion) return {};

	if (*schemaVersion < 1)
	{
		TF_WARN("Invalid schema version (%d)", *schemaVersion);
		return {};
	}

	if (*schemaVersion > JsonManifestSerializer::GetSchemaVersion())
	{
		TF_WARN("Unsupported schema version (%d)", *schemaVersion);
		return {};
	}

	const TargetUid targetUid = _ExtractTargetUid(extract, root);
	if (!targetUid.IsValid()) return {};

	Manifest manifest;

	const auto products = extract.RequireArray(root, "products");
	if (!products) return {};

	for (const pxr::JsValue& value : products->get())
	{
		if (!value.IsObject())
		{
			continue;
		}

		const pxr::JsObject& p = value.GetJsObject();
		if (p.empty())
		{
			continue;
		}

		const auto product = _ExtractProduct(extract, p);
		if (!product)
		{
			continue;
		}

		manifest.AddProduct(*product);
	}

	const std::size_t totalProducts = products->get().size();
	const std::size_t validProducts = manifest.GetProducts().size();
	const std::size_t invalidProducts = totalProducts - validProducts;

	if (validProducts == 0)
	{
		TF_WARN(
			"All products are invalid or malformed (%zu invalid products) - "
			"the resulting manifest for target (%s) is invalid."
			, invalidProducts
			, pxr::TfStringify(targetUid).c_str()
		);
		return {};
	}

	if (invalidProducts > 0)
	{
		TF_WARN(
			"Skipped (%zu) invalid or malformed products while "
			"deserializing manifest for target (%s)"
			, invalidProducts
			, pxr::TfStringify(targetUid).c_str()
		);
	}

	// "definitions" is mandatory: each entry carries the scene path and
	// permutation ops contributed by the originating TargetDefinitionEntry. Manifests
	// without it cannot reconstruct a valid TargetData, so we reject them.
	const auto definitions = extract.RequireArray(root, "definitions");
	if (!definitions) return {};

	AssetDefinitionRegistry& registry = AssetDefinitionRegistry::GetInstance();

	internal::TargetDataBuilder builder(targetUid);

	for (const pxr::JsValue& val : definitions->get())
	{
		if (!val.IsObject())
		{
			continue;
		}

		std::optional<_DecodedDefinition> decoded
			= _DefinitionFromJs(extract, val.GetJsObject());
		if (!decoded)
		{
			continue;
		}

		// Re-register the definition into the global registry. The registry
		// warns on UID conflict via _ResolveDefinitionConflict.
		registry.AddDefinition(decoded->defn);

		// Each definition entry maps 1:1 to a TargetDefinitionEntry on the
		// reconstructed TargetData (mirrors the fold done at serialize time).
		builder.AddInfo(TargetDefinitionEntry(
			decoded->defn.GetUniqueId(),
			decoded->scenePath,
			std::move(decoded->permutationOps)));
	}

	// Dependencies live inside the "target" object alongside the uid. The
	// "target" object is required (already read above to extract uid).
	const auto target = extract.RequireObject(root, "target");
	if (!target) return {};

	if (const auto dependencies = extract.GetArray(target->get(), "dependencies",
	                                                /*emptyOk*/true))
	{
		builder.SetDependencies(_TargetUidsFromJs(extract, dependencies->get()));
	}

	manifest.SetTargetData(builder.Build());

	DEBUG_MANIFEST(
		"Successfully deserialized manifest for target (%s)\n"
		, pxr::TfStringify(targetUid).c_str()
	);

	return manifest;
}

// virtual
std::string
JsonManifestSerializer::GetEncoding() const
{
	return _tokens->jsonEncoding.GetString();
}

// virtual
std::string
JsonManifestSerializer::GetFileExtension() const
{
	return _tokens->jsonSuffix.GetString();
}

// static
std::string
JsonManifestSerializer::GetSchemaName()
{
	return _tokens->schemaName.GetString();
}

// static
std::int32_t
JsonManifestSerializer::GetSchemaVersion()
{
	return _schemaVersion;
}

// static
std::string
JsonManifestSerializer::Encoding()
{
	return _tokens->jsonEncoding.GetString();
}

// static
std::string
JsonManifestSerializer::FileExtension()
{
	return _tokens->jsonSuffix.GetString();
}

// static
std::size_t
JsonManifestSerializer::MaxFileSize()
{
	return _maxFileSizeBytes;
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
