// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/jsonStoragePlugin.h"

#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/jsonManifestSerializer.h"
#include "UsdPregen/manifest.h"
#include "UsdPregen/manifestTypes.h"
#include "UsdPregen/storageOptions.h"
#include "UsdPregen/storagePlugin.h"
#include "UsdPregen/target.h"

#include "USDIncludesStart.h"
#include "pxr/base/arch/defines.h"
#include "pxr/base/arch/env.h"
#include "pxr/base/arch/errno.h"
#include "pxr/base/arch/fileSystem.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/fileUtils.h"
#include "pxr/base/tf/pathUtils.h"
#include "pxr/base/tf/safeOutputFile.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/tf/token.h"
#include "USDIncludesEnd.h"

#include <cerrno>
#include <cstdint>
#include <cstdio>
#include <string>
#include <memory>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE

PREGEN_NAMESPACE_OPEN_SCOPE

const pxr::TfToken JsonStoragePluginTokens::manifestDirEnvvar{"USDPREGEN_DEFAULT_STORAGE_DIR"};

JsonStoragePlugin::JsonStoragePlugin(const StorageOptions& options)
	: StoragePlugin(options)
	, _packageSubPathTemplate(options.packageSubPathTemplate)
{
	// TODO: move this to Initialize()
	_CheckOrCreateManifestDir(options.manifestDir);
}

// virtual
ManifestLoadResult
JsonStoragePlugin::LoadManifestPayload(const TargetUid& targetUid)
{
	using Result = ManifestLoadResult;
	using Status = ManifestLoadStatus;

	Result result;
	result.status = Status::Error;

	if (!_manifestDirReady)
	{
		result.message = "Invalid or missing manifest directory.";
		return result;
	}

	if (!targetUid)
	{
		result.message = "Invalid target uid.";
		return result;
	}

	const std::string path = GetPathForManifest(targetUid);
	const std::string absPath = pxr::TfAbsPath(path);

	FILE* inputFile = pxr::ArchOpenFile(path.c_str(), "rb");
	if (!inputFile)
	{
#if defined(ARCH_OS_WINDOWS)
		// A second check here is not ideal, however errno
		// appears to be unreliable on windows.
		if (!pxr::TfPathExists(path))
		{
			result.status = Status::DoesNotExist;
			return result;
		}

		result.message = TfStringPrintf(
			"Failed to open existing manifest file (%s)"
			, absPath.c_str());
#else
		if (errno == ENOENT) // no such file
		{
			result.status = Status::DoesNotExist;
			return result;
		}

		result.message = TfStringPrintf(
	        "Failed to open manifest file (%s): %s"
			, absPath.c_str()
			, pxr::ArchStrerror(errno).c_str());
#endif
		return result;
	}

	struct _FileCloser
	{
		void operator()(FILE* f) const
		{
			if (f)
			{
				fclose(f);
			}
		}
	};

	std::unique_ptr<FILE, _FileCloser> fileGuard(inputFile);

	const int64_t fileSize = pxr::ArchGetFileLength(inputFile);
	if (fileSize < 0)
	{
		result.message = TfStringPrintf(
							 "Failed to determine file size (%s)"
							 , absPath.c_str());
		return result;
	}

	if (fileSize == 0)
	{
		result.message = TfStringPrintf(
							 "File (%s) is empty"
							 , absPath.c_str());
		return result;
	}

	const std::size_t numBytes = static_cast<std::size_t>(fileSize);
	const std::size_t maxBytes = JsonManifestSerializer::MaxFileSize();
	if (numBytes > maxBytes)
	{
		result.message = TfStringPrintf(
			"Manifest file is too large (%s, %zu bytes, max %zu)"
			, absPath.c_str()
			, numBytes
			, maxBytes);
		return result;
	}

	result.payload.data.resize(numBytes);

	const int64_t bytesRead = pxr::ArchPRead(inputFile,
											 result.payload.data.data(),
											 numBytes,
											 /*offset=*/0);

	if (bytesRead < 0 || static_cast<std::size_t>(bytesRead) != numBytes)
	{
		result.payload.data.clear();
		result.message = TfStringPrintf(
		    "Failed to read %zu bytes from file (%s)"
		    , numBytes
			, absPath.c_str());
		return result;
	}

	result.payload.encoding = JsonManifestSerializer::Encoding();
	result.status = Status::Loaded;

	return result;
}

// virtual
ManifestSaveResult
JsonStoragePlugin::StoreManifestPayload(const TargetUid& targetUid,
										   const ManifestPayload& payload)
{
	using Result = ManifestSaveResult;
	using Status = ManifestSaveStatus;

	Result result;
	result.status = Status::Error;

	if (!_manifestDirReady)
	{
		result.message = "Invalid or missing manifest directory.";
		return result;
	}

	if (!targetUid || payload.encoding.empty() || payload.data.empty())
	{
		result.message = "Invalid target uid and/or manifest payload";
		return result;
	}

	if (payload.encoding != JsonManifestSerializer::Encoding())
	{
		result.message = TfStringPrintf(
							 "Unsupported manifest encoding (%s)"
							 , payload.encoding.c_str());
		return result;
	}

	if (payload.data.size() > JsonManifestSerializer::MaxFileSize())
	{
		result.message = TfStringPrintf(
			                 "Payload size is too large (%zu bytes, max %zu)"
			                 , payload.data.size()
		                     , JsonManifestSerializer::MaxFileSize());
		return result;
	}

	const std::string path = GetPathForManifest(targetUid);
	const std::string absPath = pxr::TfAbsPath(path);

	// For now we never attempt to overwrite an existing manifest.
	if (TfIsFile(path, /*resolveSymlinks=*/true))
	{
		result.status = Status::NotSaved;
		return result;
	}
	else if (TfPathExists(path, /*resolveSymlinks=*/true))
	{
		result.message = TfStringPrintf(
			                 "Path (%s) exists but is not a file."
			                 , absPath.c_str());
		return result;
	}

	// TODO: Replace TfSafeOutputFile with an implementation that guarantees
	// atomic creation and prevents overwriting an existing manifest.
	//
	// The desired behavior is "create-if-not-exists" semantics so that multiple
	// processes attempting to write the same manifest cannot overwrite one
	// another.
	//
	// Currently we rely on TfSafeOutputFile only for its temporary-file +
	// rename behavior. However, TfSafeOutputFile::Close() always replaces the
	// destination file, even if it already exists. This means that if another
	// process creates the manifest inbetween the earlier exists() check and the
	// call to Close(), the existing manifest will be overwritten.

	pxr::TfSafeOutputFile safeFile = pxr::TfSafeOutputFile::Replace(path);
	FILE* outputFile = safeFile.Get();
	if (!outputFile)
	{
		result.message = TfStringPrintf(
							 "Failed to open file (%s) for writing."
							 , absPath.c_str());
		return result;
	}

	const int64_t bytesWritten = pxr::ArchPWrite(outputFile,
												 payload.data.data(),
												 payload.data.size(),
												 /*offset=*/0);
	if (bytesWritten < 0
		|| static_cast<std::size_t>(bytesWritten) != payload.data.size())
	{
		safeFile.Discard();
		result.message = TfStringPrintf(
							 "Failed to write (%zu) bytes to file (%s)"
							 , payload.data.size()
							 , absPath.c_str());
		return result;
	}

	if (fflush(outputFile) != 0)
	{
		safeFile.Discard();
		result.message = TfStringPrintf(
							 "Failed to flush file (%s)"
							 , absPath.c_str());
		return result;
	}

	safeFile.Close();

	DEBUG_MANIFEST(
		"Saved manifest to file (%s) (JSON storage plugin)"
		, absPath.c_str()
	);

	result.status = Status::Saved;
	return result;
}

// virtual
ManifestSaveResult
JsonStoragePlugin::PersistManifestPayload(const TargetUid& /*targetUid*/)
{
	// Filesystem-backed: StoreManifestPayload already wrote the JSON to disk,
	// so there is nothing to do at the persist phase. We override only to
	// avoid the StoragePlugin base default which returns Status::Error.
	ManifestSaveResult result;
	result.status = ManifestSaveStatus::NotSaved;
	return result;
}

// virtual
ManifestPayload
JsonStoragePlugin::SerializeManifest(const Manifest& manifest)
{
	ManifestPayload payload;

	JsonManifestSerializer serializer;

	if (!serializer.Serialize(manifest, payload))
	{
		payload.encoding.clear();
		payload.data.clear();
	}

	return payload;
}

// virtual
Manifest
JsonStoragePlugin::DeserializeManifestPayload(const ManifestPayload& payload)
{
	JsonManifestSerializer serializer;

	if (payload.encoding != JsonManifestSerializer::Encoding())
	{
		TF_RUNTIME_ERROR(
			"Unsupported manifest encoding (%s)"
			, payload.encoding.c_str());
		return Manifest();
	}

	return serializer.Deserialize(payload);
}

// virtual
std::string
JsonStoragePlugin::GetNameForUAsset(
	const TargetUid& targetUid,
	const std::vector<const ExtAssetDefinition*>& definitions,
	const std::string& assetType)
{
	// Use the default Interchange asset naming by default.
	return {};
}

// virtual
std::string
JsonStoragePlugin::GetPackageSubPathForUAsset(
	const TargetUid& targetUid,
	const std::vector<const ExtAssetDefinition*>& definitions,
	const std::string& assetType)
{
	if (!TF_VERIFY(targetUid))
	{
		return {};
	}

	if (!TF_VERIFY(!definitions.empty()))
	{
		return {};
	}

	const std::string& effectiveTemplate = _packageSubPathTemplate.empty()
		? StoragePlugin::DefaultPackageSubPathTemplate()
		: _packageSubPathTemplate;

	return StoragePlugin::ResolvePackageSubPathTemplate(
		effectiveTemplate, targetUid, definitions, assetType);
}

void
JsonStoragePlugin::_CheckOrCreateManifestDir(const std::string& inDir)
{
	const std::string manifestDirEnvvar
		= JsonStoragePluginTokens::manifestDirEnvvar.GetString();

	// Resolution order:
	//   1. USDPREGEN_DEFAULT_STORAGE_DIR environment variable
	//   2. inDir (typically from StorageOptions::manifestDir)
	//   3. "<HOME>/UE_UsdPregen_Manifests" (built-in fallback)
	const std::string manifestDir = [&]()
	{
		if (pxr::ArchHasEnv(manifestDirEnvvar))
		{
			const std::string path = pxr::ArchExpandEnvironmentVariables(
									     pxr::ArchGetEnv(manifestDirEnvvar));
			TF_STATUS(
				"[USDPregen] Using manifest storage directory from "
				"environment variable %s - (%s)"
				, manifestDirEnvvar.c_str()
				, path.c_str()
			);
			return path;
		}

		if (!inDir.empty())
		{
			return inDir;
		}

		// Final fallback: "<HOME>/UE_UsdPregen_Manifests".
#if defined(ARCH_OS_WINDOWS)
		const std::string homeDir = pxr::ArchGetEnv("USERPROFILE");
#else
		const std::string homeDir = pxr::ArchGetEnv("HOME");
#endif
		if (homeDir.empty())
		{
			return std::string{};
		}
		return pxr::TfNormPath(pxr::TfStringCatPaths(homeDir, "UE_UsdPregen_Manifests"));
	}();

	if (manifestDir.empty())
	{
		TF_WARN("No manifest directory specified.");
		return;
	}

	const std::string absPath = pxr::TfAbsPath(manifestDir);

	if (pxr::TfPathExists(absPath))
	{
		if (!pxr::TfIsDir(absPath))
		{
			TF_WARN(
				"Invalid manifest directory (%s) - path exists but is not a directory."
				, absPath.c_str()
			);
			return;
		}
	}
	else
	{
		if (!pxr::TfMakeDirs(absPath, -1, /*existOk=*/true))
		{
			TF_WARN(
				"Failed to create manifest directory (%s)"
				, absPath.c_str()
			);
			return;
		}
	}

	_manifestDir = absPath;
	_manifestDirReady = true;
}

// virtual
std::string
JsonStoragePlugin::GetPathForManifest(const TargetUid& targetUid)
{
	if (!_manifestDirReady || !targetUid)
	{
		return {};
	}

	// TODO: full sanitize filename or hash. For now just handle slashes.
	std::string uidStr = pxr::TfStringify(targetUid);
	for (char& c : uidStr)
	{
		if (c == '/' || c == '\\')
		{
			c = '%';
		}
	}

	return pxr::TfNormPath(pxr::TfStringCatPaths(
				    _manifestDir,
			        uidStr + JsonManifestSerializer::FileExtension()));
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
