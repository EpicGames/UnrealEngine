// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "layerWriter.h"

#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/permutationOps.h"
#include "UsdPregen/primPermutation.h"
#include "UsdPregen/target.h"

#include "item.h"
#include "persistentHasher.h"
#include "util.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/enum.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/staticTokens.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/sdf/attributeSpec.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/sdf/spec.h"
#include "pxr/usd/sdf/primSpec.h"
#include "pxr/usd/sdf/relationshipSpec.h"
#include "pxr/usd/sdf/variantSpec.h"
#include "pxr/usd/sdf/variantSetSpec.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/primFlags.h"
#include "pxr/usd/usd/relationship.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usd/stagePopulationMask.h"
#include "pxr/usd/usd/variantSets.h"
#include "pxr/usd/usd/inherits.h"
#include "USDIncludesEnd.h"

#include <cinttypes>
#include <cstddef>
#include <string>
#include <optional>
#include <variant>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE // for TF macros

// clang-format off
TF_DEFINE_PRIVATE_TOKENS(
	_tokens,
	((assetDefinitionTypeName, "AssetDefinition"))
	(assetMetadata)
	(assets)
	(content)
	((defnUidAttr, "pregen:target:definitionUid"))
	((dependsOnRel, "pregen:dependsOn"))
	((encapsulatedPathsAttr, "pregen:subpaths:encapsulatedDefinitions"))
	((unencapsulatedPathsAttr, "pregen:subpaths:unencapsulatedDefinitions"))
	((identifierAttr, "pregen:extasset:identifier"))
	(info)
	((nameAttr, "pregen:extasset:name"))
	(ops)
	((opPrefix, "op_"))
	((overridesAttr, "pregen:overrides"))
	((pathAtIntroductionAttr, "pregen:pathAtIntroduction"))
	(permutations)
	((pidUidAttr, "pregen:target:permutationUid"))
	((pidHashEntriesAttr, "pregen:permutationUidHashEntries"))
	((pidPrefix, "pid"))
	((scenePathAttr, "pregen:scenePath"))
	((targetTypeName, "Target"))
	((targetRel, "pregen:target"))
	(variantSet)
	((versionAttr, "pregen:extasset:version"))
	((uidAttr, "pregen:extasset:uid"))
);
// clang-format on

PREGEN_NAMESPACE_OPEN_SCOPE

using namespace internal::util;

namespace
{

const pxr::SdfPath _definitionsRootPath{"/__definitions__"};
const pxr::SdfPath _targetsRootPath{"/__targets__"};
const pxr::SdfPath _sceneRootPath{"/__scene__"};

void
_ReportAttrFailure(const pxr::UsdPrim& prim, const pxr::TfToken& name)
{
	TF_VERIFY(false,
		"Failed to read attribute or value '%s' on prim <%s>, "
		"resulting tracker and target data may be invalid.\n"
		, name.GetText()
		, prim.GetPath().GetText()
	);
}

template <typename T>
std::optional<T> _GetAttrValueImpl(
	const pxr::UsdPrim& prim,
	const pxr::TfToken& name,
	bool emptyOk = false)
{
	if (prim) {
		if (pxr::UsdAttribute attr = prim.GetAttribute(name)) {
			if (T value; attr.Get(&value)) {
				return value;
			}
		}
		if (emptyOk) {
			return T{};
		}
	}

	return std::nullopt;
}

template <typename T>
T _GetAttrValue(
	const pxr::UsdPrim& prim,
	const pxr::TfToken& name,
	bool emptyOk = false)
{
	if (auto v = _GetAttrValueImpl<T>(prim, name, emptyOk)) {
		return std::move(*v);
	}

	_ReportAttrFailure(prim, name);
	return T{};
}

pxr::SdfPathSet
_GetStringArrayAttrAsPathSet(
	const pxr::UsdPrim& prim,
	const pxr::TfToken& attrName)
{
	pxr::SdfPathSet pathSet;

	const VtStringArray strings
		= _GetAttrValue<pxr::VtStringArray>(prim, attrName);

	for (const std::string& str : strings)
	{
		pxr::SdfPath path(str);
		if (TF_VERIFY(!path.IsEmpty()))
		{
			pathSet.insert(std::move(path));
		}
	}

	return pathSet;
}

// Builds a SerializedOp from a composed UsdPrim representing a single
// permutation op.
//
// The op data (typeName + opargs:* attributes) is authored inside variant
// specs in the data layer. Looking up the underlying SdfPrimSpec at the op
// prim's path yields only the flat scaffolding (scenePath) - the variant-
// selected typeName and opargs are only visible on the composed UsdPrim.
SerializedOp
_BuildSerializedOpFromPrim(const pxr::UsdPrim& opPrim)
{
	SerializedOp result;

	result.typeName = opPrim.GetTypeName().GetString();

	constexpr const char* opargsPrefix = "opargs:";
	constexpr std::size_t opargsPrefixLen = 7;

	for (const pxr::UsdAttribute& attr : opPrim.GetAttributes())
	{
		const std::string name = attr.GetName().GetString();
		if (name.size() <= opargsPrefixLen
			|| name.compare(0, opargsPrefixLen, opargsPrefix) != 0)
		{
			continue;
		}

		pxr::VtValue value;
		if (!attr.Get(&value))
		{
			continue;
		}

		SerializedOpArg arg;
		arg.name = name.substr(opargsPrefixLen);
		arg.typeName = attr.GetTypeName().GetAsToken().GetString();
		if (value.IsHolding<std::string>())
		{
			arg.value = value.UncheckedGet<std::string>();
		}
		else
		{
			arg.value = pxr::TfStringify(value);
		}

		result.args.push_back(std::move(arg));
	}

	return result;
}

// Walks the children of <initialDefn>/ops and reconstructs a PermutationOp
// for each via the typeName-keyed factory registry on PermutationOp.
//
// Each child is treated as a composed UsdPrim representing a single op
// (typeName + opargs:* attrs). Children whose typeName has no registered
// factory are skipped (with a warning logged by the registry).
PermutationOpVector
_ExtractOpsFromDefn(const pxr::UsdPrim& initialDefn)
{
	PermutationOpVector ops;

	if (!initialDefn)
	{
		return ops;
	}

	const pxr::UsdPrim opsPrim = initialDefn.GetChild((_tokens->ops));
	if (!opsPrim)
	{
		return ops;
	}

	for (const pxr::UsdPrim& childPrim : opsPrim.GetAllChildren())
	{
		SerializedOp serialized = _BuildSerializedOpFromPrim(childPrim);
		if (serialized.typeName.empty())
		{
			continue;
		}

		if (PermutationOpRefPtr op = FromSerializedOp(serialized))
		{
			ops.push_back(std::move(op));
		}
	}

	return ops;
}

bool
_ApplyOpsToLayer(const pxr::UsdPrim& initialDefn,
	             pxr::SdfLayerRefPtr permOverlayLayer)
{
	pxr::UsdPrim opsPrim = initialDefn.GetChild((_tokens->ops));
	if (!opsPrim)
	{
		return false;
	}

	for (const UsdPrim& childPrim : opsPrim.GetAllChildren())
	{
		const std::string opPathString
			= _GetAttrValue<std::string>(childPrim, _tokens->scenePathAttr);
		if (!TF_VERIFY(SdfPath::IsValidPathString(opPathString)))
		{
			continue;
		}

		const pxr::SdfPath opPath{opPathString};
		if (!TF_VERIFY(opPath.IsPrimPath()))
		{
			continue;
		}

		if (childPrim.GetTypeName() == "UsdVariantSelectionOp")
		{
			const std::string variantSetName
				= _GetAttrValue<std::string>(childPrim, TfToken{"opargs:variantSet"});
			const std::string variantName
				= _GetAttrValue<std::string>(childPrim, TfToken{"opargs:variant"});
			if (!TF_VERIFY(!variantSetName.empty() && !variantName.empty()))
			{
				continue;
			}

			pxr::SdfPrimSpecHandle variantPrimSpec
				= pxr::SdfCreatePrimInLayer(permOverlayLayer, opPath);
			variantPrimSpec->SetVariantSelection(variantSetName, variantName);
		}
		else if (childPrim.GetTypeName() == "UsdInheritOp")
		{
			const std::string listOpTypeStr
				= _GetAttrValue<std::string>(childPrim, TfToken{"opargs:listOpType"});

			bool found = false;
			pxr::SdfListOpType listOpType
				= pxr::TfEnum::GetValueFromName<pxr::SdfListOpType>(listOpTypeStr, &found);
			if (!found)
			{
				TF_WARN(
					"Invalid inherit list op type (%s) - defaulting to 'prepended'",
					listOpTypeStr.c_str()
				);
				listOpType = pxr::SdfListOpTypePrepended;
			}

			const std::string pathToInheritStr
				= _GetAttrValue<std::string>(childPrim, TfToken{"opargs:pathToInherit"});

			if (!TF_VERIFY(SdfPath::IsValidPathString(pathToInheritStr)))
			{
				continue;
			}

			pxr::SdfPath pathToInherit{ pathToInheritStr };
			pxr::SdfPrimSpecHandle inheritPrimSpec
				= pxr::SdfCreatePrimInLayer(permOverlayLayer, opPath);

			pxr::SdfPathListOp op;
			pxr::VtValue value = inheritPrimSpec->GetInfo(pxr::SdfFieldKeys->InheritPaths);
			if (value.IsHolding<pxr::SdfPathListOp>())
			{
				op = value.UncheckedGet<pxr::SdfPathListOp>();
			}

			switch (listOpType)
			{
			case pxr::SdfListOpTypeExplicit:
			{
				pxr::SdfPathVector items = op.GetExplicitItems();
				items.push_back(pathToInherit);
				op.SetExplicitItems(items);
				break;
			}
			case pxr::SdfListOpTypePrepended:
			{
				pxr::SdfPathVector items = op.GetPrependedItems();
				items.push_back(pathToInherit);
				op.SetPrependedItems(items);
				break;
			}
			case pxr::SdfListOpTypeAppended:
			{
				pxr::SdfPathVector items = op.GetAppendedItems();
				items.push_back(pathToInherit);
				op.SetAppendedItems(items);
				break;
			}
			case pxr::SdfListOpTypeAdded:
			{
				pxr::SdfPathVector items = op.GetAddedItems();
				items.push_back(pathToInherit);
				op.SetAddedItems(items);
				break;
			}
			case pxr::SdfListOpTypeDeleted:
			{
				pxr::SdfPathVector items = op.GetDeletedItems();
				items.push_back(pathToInherit);
				op.SetDeletedItems(items);
				break;
			}
			case pxr::SdfListOpTypeOrdered:
			{
				pxr::SdfPathVector items = op.GetOrderedItems();
				items.push_back(pathToInherit);
				op.SetOrderedItems(items);
				break;
			}
			default:
				break;
			}

			inheritPrimSpec->SetInfo(pxr::SdfFieldKeys->InheritPaths, VtValue(op));
		}
	}

	return true;
}

} // anonymous namespace

namespace internal
{

LayerWriter::LayerWriter(const pxr::UsdStageRefPtr& stage)
	: _dataLayer(pxr::SdfLayer::CreateAnonymous())
	, _stage(stage)
{
}

ScopeHandle
LayerWriter::AssetDefinitionAdded(const ExtAssetDefinition& assetDefn)
{
	// Create the main definition spec for the asset
	const TfToken assetUidToken {
		TfMakeValidIdentifier(assetDefn.GetUniqueId())
	};

	const pxr::SdfPath assetSpecPath = _definitionsRootPath.AppendChild(assetUidToken);
	if (_dataLayer->HasSpec(assetSpecPath))
	{
		return ScopeHandle { assetSpecPath };
	}

	const pxr::SdfPrimSpecHandle assetSpec = _CreatePrimInDataLayer(assetSpecPath);
	assetSpec->SetTypeName(_tokens->assetDefinitionTypeName.GetString());

	// Create the info prim
	const pxr::SdfPath infoSpecPath = assetSpecPath.AppendChild(_tokens->info);
	const pxr::SdfPrimSpecHandle infoSpec  = _CreatePrimInDataLayer(infoSpecPath);

	// Write the asset uid attribute to the info prim
	pxr::SdfAttributeSpec::New(
		infoSpec,
		_tokens->uidAttr,
		pxr::SdfValueTypeNames->String)
			->SetDefaultValue(VtValue(assetDefn.GetUniqueId()));

	// Write the asset name attribute
	pxr::SdfAttributeSpec::New(
		infoSpec,
		_tokens->nameAttr,
		pxr::SdfValueTypeNames->String)
			->SetDefaultValue(VtValue(assetDefn.GetName()));

	// Write the identifier attribute
	pxr::SdfAttributeSpec::New(
	    infoSpec,
		_tokens->identifierAttr,
	    pxr::SdfValueTypeNames->Asset)
		    ->SetDefaultValue(VtValue(assetDefn.GetIdentifier()));

	// Write the version (optional)
	auto versionSpec = pxr::SdfAttributeSpec::New(
						   infoSpec,
						   _tokens->versionAttr,
						   pxr::SdfValueTypeNames->String);

	const std::string& version = assetDefn.GetVersion();
	if (!version.empty())
	{
		versionSpec->SetDefaultValue(VtValue(version));
	}

	// Transfer metadata, if specified.
	if (const std::optional<VtDictionary>& metadata = assetDefn.GetMetadata())
	{
		infoSpec->SetCustomData(_tokens->assetMetadata, VtValue(*metadata));
	}

	return ScopeHandle { assetSpec->GetPath() };
}

ScopeHandle
LayerWriter::AddPermutation(
	const Item* curItem,
	const pxr::SdfPath& primPath,
	const PrimPermutationConstRefPtr& perm,
	types::int32 nestingLevel)
{
	TF_AXIOM(curItem);

	const ItemScope* curScope = curItem->GetCurrentScope();

	const std::string nextIndex = std::to_string(nestingLevel);
	const std::string variantSetName = _tokens->permutations.GetString() + nextIndex;
	std::string variantName = perm->GetUniqueId();

	// If the prim with the permutation isn't the item prim we're tracking its
	// necessary to put the path to the prim in the permutation variant name
	// so that it's unique.
	//
	// TODO: right now this just applies the prim name on the end to get the
	// tests to pass - for deeper paths we would need a hash of some sort.
	if (primPath != curItem->GetPath())
	{
		variantName = variantName + "_" + primPath.GetName();
	}

	DEBUG_PERMUTATION(
		"Pushing permutation (%s), level (%s), on <%s>\n"
		, variantName.c_str()
		, nextIndex.c_str()
		, primPath.GetText()
	);

	const pxr::SdfSpecHandle curSpec = _GetPrimAtPath(curScope->GetHandle());

	const pxr::SdfVariantSetSpecHandle variantSetSpec
		= util::GetOrCreateVariantSetSpec(curSpec, variantSetName);

	const pxr::SdfVariantSpecHandle variantSpec
		= internal::util::GetOrCreateVariantSpec(variantSetSpec, variantName,
			[&curItem, &primPath, &perm, &nextIndex]
			(const pxr::SdfVariantSpecHandle& variantSpec)
	{
		// If this is the first time creating this variant write each op in
		// the permutation set as a prim below the ops scope.
		const pxr::SdfPrimSpecHandle opsHolderSpec
			= pxr::SdfPrimSpec::New(variantSpec->GetPrimSpec(),
									_tokens->ops,
									pxr::SdfSpecifierOver);

		const std::string opPrefix = _tokens->opPrefix.GetString() + nextIndex + "_";
		int opNum = 1;
		for (const PermutationOpRefPtr& op : perm->GetOps())
		{
			const std::string opSpecName = opPrefix + std::to_string(opNum++);
			pxr::SdfPrimSpecHandle opSpec = pxr::SdfPrimSpec::New(
												opsHolderSpec,
												opSpecName,
												pxr::SdfSpecifierOver);

			// TODO rename this to WriteToSpec or something
			op->Serialize(opSpec);

			const pxr::SdfPath opScenePath = primPath.ReplacePrefix(
				curItem->GetPath(), curItem->GetPathAtIntroduction());

			pxr::SdfAttributeSpec::New(opSpec,
									   _tokens->scenePathAttr,
									   pxr::SdfValueTypeNames->String)
				->SetDefaultValue(VtValue(opScenePath.GetString()));
		}
	});

	return ScopeHandle { variantSpec->GetPath() };
}

void LayerWriter::AddChildAssetDependencyToParent(const Item* curItem)
{
	TF_AXIOM(curItem);

	if (curItem->IsRootItem())
	{
		return;
	}

	const Item* parentItem = curItem->GetParentItem();

	const pxr::SdfSpecHandle curSpec = _GetPrimAtPath(curItem->GetCurrentScope()->GetHandle());
	const pxr::SdfSpecHandle parentSpec = _GetPrimAtPath(parentItem->GetCurrentScope()->GetHandle());

	const pxr::SdfPrimSpecHandle assetsSpec = _GetAssetsContainerSpec(parentSpec);

	// TODO for now use TfMakeValidIdentifier to replace any invalid characters
	// in the uid with an _ so that we can create a prim. This could lead to
	// collisions however so we may need to encode more carefully.
	const pxr::SdfPath assetRefPath = assetsSpec->GetPath().AppendChild(
	   TfToken(TfMakeValidIdentifier(curItem->GetUniqueId())));

	if (!_dataLayer->HasSpec(assetRefPath))
	{
		pxr::SdfPrimSpecHandle assetRefSpec = _CreatePrimInDataLayer(assetRefPath);
		pxr::SdfPathListOp op;
		op.SetExplicitItems({curSpec->GetPath()});
		assetRefSpec->SetInfo(pxr::SdfFieldKeys->InheritPaths, VtValue(op));
		assetRefSpec->SetActive(false);

		DEBUG_TARGET("Added dependency <%s> to parent <%s>\n"
			, curSpec->GetPath().GetText()
			, parentSpec->GetPath().GetText());
	}
}

void
LayerWriter::AddNewScenePath(const pxr::SdfPath& path)
{
	// Create the scene root path if it doesn't exist
	pxr::SdfPrimSpecHandle sceneRootSpec = _GetPrimAtPath(_sceneRootPath);
	if (!sceneRootSpec)
	{
		sceneRootSpec = _CreatePrimInDataLayer(_sceneRootPath);
		if (!TF_VERIFY(sceneRootSpec))
		{
			return;
		}
		sceneRootSpec->SetSpecifier(pxr::SdfSpecifierDef);
	}

	if (path == pxr::SdfPath::AbsoluteRootPath())
	{
		return;
	}

	const pxr::SdfPath scenePath = sceneRootSpec->GetPath().AppendPath(path
		.MakeRelativePath(pxr::SdfPath::AbsoluteRootPath()));

	const pxr::SdfPrimSpecHandle sceneSpec = _CreatePrimInDataLayer(scenePath);
	if (!TF_VERIFY(sceneSpec))
	{
		return;
	}

	sceneSpec->SetSpecifier(pxr::SdfSpecifierDef);

	if (const pxr::UsdPrim& srcPrim = _stage->GetPrimAtPath(path))
	{
		const pxr::TfToken& srcTypeName = srcPrim.GetTypeName();
		if (!srcTypeName.IsEmpty())
		{
			sceneSpec->SetTypeName(srcPrim.GetTypeName());
		}
	}

	// TODO: if/when the scene path above eventually represents a target, create
	// and connect a relationship (_tokens->targetRel) to the target when
	// it gets created.
}

bool
LayerWriter::Save(const std::string& filename)
{
	return _dataLayer && _dataLayer->Export(filename);
}

pxr::SdfPrimSpecHandle
LayerWriter::_CreatePrimInDataLayer(const pxr::SdfPath& primPath)
{
	return pxr::SdfCreatePrimInLayer(_dataLayer, primPath);
}

pxr::SdfSpecHandle
LayerWriter::_GetPrimAtPath(const ScopeHandle& handle)
{
	return _dataLayer->GetPrimAtPath(handle.id);
}

pxr::SdfPrimSpecHandle
LayerWriter::_GetPrimAtPath(const pxr::SdfPath& primPath) const
{
	if (TF_VERIFY(primPath.IsPrimPath()))
	{
		return _dataLayer->GetPrimAtPath(primPath);
	}

	return {};
}

pxr::SdfPrimSpecHandle
LayerWriter::_GetAssetsContainerSpec(const pxr::SdfSpecHandle& parentSpec)
{
	const pxr::SdfPath assetsSpecPath
		= parentSpec->GetPath().AppendChild(_tokens->assets);
	if (!_dataLayer->HasSpec(assetsSpecPath))
	{
		return _CreatePrimInDataLayer(assetsSpecPath);
	}

	return _dataLayer->GetPrimAtPath(assetsSpecPath);
}

std::optional<TargetUid>
LayerWriter::GenerateOrReuseTarget(const pxr::TfSpan<Item*>& itemStack)
{
	if (!TF_VERIFY(!itemStack.empty(), "Item stack is empty\n"))
	{
		return std::nullopt;
	}

	Item* curItem = itemStack.back();
	if (!TF_VERIFY(curItem, "Current item stack entry is null\n"))
	{
		return std::nullopt;
	}

	ItemScope* curScope = curItem->GetCurrentScope();
	if (!TF_VERIFY(curScope, "Current item scope is null\n"))
	{
		return std::nullopt;
	}

	if (!curScope->HasContent())
	{
		DEBUG_TARGET(
			 "Skipping target for asset %s (scope <%s> has "
			 "no content at namespace location <%s>)\n"
			 , curItem->GetUniqueId().c_str()
			 , curScope->GetHandle().id.GetText()
			 , curScope->GetPath().GetText()
		);

		return std::nullopt;
	}

	DEBUG_TARGET(
	    "Generating target for asset (%s) at namespace location <%s> ...\n"
		, curItem->GetUniqueId().c_str()
		, curScope->GetPath().GetText()
	);

	// Get the override depth for the current item. The override depth tells us
	// which (possibly ancestor) asset contributes opinions and so which asset
	// we must run the eventual import from. For example, imagine three nested
	// assets A -> B - > C, and the current item represents C. The depth of
	// each asset is A=0, B=1, C=2. If the override depth is 2, it means we can
	// schedule and import C using C.usd, since there are no opinions coming
	// from either A or B. If the override depth is 1, we must schedule C using
	// B.usd, since B has opinions that may alter C. Lastly, if the override
	// depth is 0 we must import using A.usd to capture whatever A is
	// contributing.
	//
	// As we do this, we must also ensure that the relevant permutations are
	// correctly configured at each asset boundary.
	//
	// Lastly, if the current scope is getting consumed by an ancestor, start
	// the override depth at the appropriate index.
	//
	const bool isConsumingDescendants
		= curScope->GetConsumerDepth() != types::InvalidIndex;

	const types::int32 overrideDepth = isConsumingDescendants
		                             ? curScope->GetConsumerDepth()
									 : curItem->GetOverrideDepth();

	const bool isEncapsulated = curItem->GetDepth() == overrideDepth;

	if (!TF_VERIFY(
			overrideDepth >= 0
			    && static_cast<size_t>(overrideDepth) < itemStack.size()
			, "Invalid override depth (%d) encountered for scope <%s> "
			  "and current stack size (%zu)"
			, overrideDepth
			, curScope->GetPath().GetText()
			, itemStack.size()))
	{
		return std::nullopt;
	}

	DEBUG_TARGET(
		"Item depth is %d, override depth is %d\n"
		, curItem->GetDepth()
		, overrideDepth
	);

	// Create our target spec with a temporary name for now. We'll rename it at
	// the end once we've generated a hash from the specific permutation
	// combinations, override paths etc for this target.
	const TfToken targetSpecName {"t" + std::to_string(util::RandomUInt64())};
	const pxr::SdfPath targetSpecPath = _targetsRootPath.AppendChild(targetSpecName);
	const pxr::SdfPrimSpecHandle targetSpec = _CreatePrimInDataLayer(targetSpecPath);
	targetSpec->SetTypeName(_tokens->targetTypeName.GetString());

	// mutable because we may need to promote the content (boundables etc)
	Item* rootItem = itemStack[overrideDepth];

	DEBUG_TARGET("Root item uid is %s\n", rootItem->GetUniqueId().c_str());

	{
		// Add the initial inherits arc pointing to the root item
		pxr::SdfPathListOp op;
		op.SetExplicitItems({rootItem->GetPrimScope()->GetHandle().id});
		targetSpec->SetInfo(pxr::SdfFieldKeys->InheritPaths, VtValue(op));
	}

	// Add the dependency list from the current scope to the
	// "dependsOn" relationship on the info spec.
	if (curScope->HasDependencies())
	{
		const pxr::SdfPrimSpecHandle infoSpec
			= _CreatePrimInDataLayer(targetSpecPath.AppendChild(_tokens->info));
		auto rel = pxr::SdfRelationshipSpec::New(infoSpec, _tokens->dependsOnRel);
		auto targetPathList = rel->GetTargetPathList();
		for (const pxr::SdfPath& path : curScope->GetDependencies())
		{
			targetPathList.Append(path);
		}
	}

	// Add the current path to the parents list of definition paths.
	if (Item* parentItem = curItem->GetParentItem())
	{
		parentItem->GetCurrentScope()
	        ->AddDescendantDefinitionScenePath(
			    curItem->GetPath(), isEncapsulated);
	}

	// Write the current encapsulated and unencapsulated definition paths to
	// the content prim. Note we write an empty array if needed for clarity.
	auto _WriteDefinitionPaths = [this, &curItem, &targetSpecPath]
		(const pxr::SdfPathSet& pathSet, const pxr::TfToken& attrName)
	{
		pxr::VtStringArray strings;
		strings.reserve(pathSet.size());

		const pxr::SdfPrimSpecHandle contentSpec
			= _CreatePrimInDataLayer(targetSpecPath.AppendChild(_tokens->content));

		std::transform(pathSet.begin(), pathSet.end(),
			std::back_inserter(strings),
			[&curItem](const pxr::SdfPath& path)
			{
				return path.ReplacePrefix(
					       curItem->GetPath(),
					       curItem->GetPathAtIntroduction()
					   ).GetString();
			});

		pxr::SdfAttributeSpec::New(
			contentSpec,
			attrName,
			pxr::SdfValueTypeNames->StringArray
		)->SetDefaultValue(VtValue(strings));
	};

	_WriteDefinitionPaths(curScope->GetEncapsulatedDefinitionPaths(),
		                  _tokens->encapsulatedPathsAttr);
	_WriteDefinitionPaths(curScope->GetUnencapsulatedDefinitionPaths(),
		                  _tokens->unencapsulatedPathsAttr);

	// Build a list of permutation entries for this target. Each permutation
	// must have a unique string representation that can be used as a source
	// for the hash.
	PersistentHasher permHasher;

	// Check that the path targeted by the asset reference matches the referenced
	// layer default prim. If it doesn't we must add the referenced path to the
	// permutation hash so that the same asset utilized multiple times but
	// referencing different prims will each generate a unique target.
	//
	// This runs for root items (depth == 0) too: when the asset layer is opened
	// directly (e.g. worker discovery), the asset prim ends up at the root, but
	// Item::GetPathAtIntroduction/GetDefaultPrim still report the prim's own
	// scene path and the layer's authored defaultPrim (see
	// util::GetIntroPathAndDefaultPrim), so this still works. 
	// Skipping the root would produce a different permutation UID from submission 
	// discovery, where the same asset is reached via a reference at depth > 0.
	{
		const TfToken& defaultPrim = curItem->GetDefaultPrim();
		const pxr::SdfPath& introPath = curItem->GetPathAtIntroduction();
		if (defaultPrim.IsEmpty() ||
			pxr::SdfPath::AbsoluteRootPath().AppendChild(defaultPrim) != introPath)
		{
			permHasher.AddString(TfStringPrintf("introPath=%s", introPath.GetText()));
		}
	}

	pxr::SdfPrimSpecHandle curTarget = targetSpec;

	for (size_t itemIdx = overrideDepth; itemIdx < itemStack.size(); ++itemIdx)
	{
		const Item* thisItem = itemStack[itemIdx];

		TF_VERIFY(thisItem->GetPath().HasPrefix(rootItem->GetPath()));

		DEBUG_TARGET(
			"Processing asset %s %zu/%zu ...\n"
			, thisItem->GetUniqueId().c_str()
			, itemIdx+1
			, itemStack.size()
		);

		if (thisItem != curItem)
		{
			permHasher.AddString(thisItem->GetUniqueId());
		}

		// For each variant permutation scope (i.e exclude the first scope
		// which is the prim spec for the item), set the correct permutation
		// (variant) and add to the list of hash entries.
		for (size_t scopeIdx = 1; scopeIdx < thisItem->NumScopes(); ++scopeIdx)
		{
			_ConfigureTargetOpsForLevel(
				itemStack,
				curTarget,
				itemIdx,
				scopeIdx,
				overrideDepth,
				permHasher);
		}

		_ConfigureTarget(itemStack, curTarget, itemIdx, overrideDepth, permHasher);

		// If this is not the last item, active the next asset.
		if (itemIdx < itemStack.size() - 1)
		{
			const pxr::SdfPath assetsPath = curTarget->GetPath().AppendChild(_tokens->assets);
			const pxr::SdfPath nextAssetPath = assetsPath.AppendChild(
				TfToken(TfMakeValidIdentifier(itemStack[itemIdx+1]->GetUniqueId())));
			// Increment to the next target and activate
			curTarget = _CreatePrimInDataLayer(nextAssetPath);
			curTarget->SetActive(true);
		}
	}

	// Add the override paths from the current scope.
	const TfSpan<const pxr::SdfPath> overridePaths = curScope->GetOverridePaths();
	for (const pxr::SdfPath& overridePath : overridePaths)
	{
		permHasher.AddString(overridePath.GetString());
	}

	// Build up the target name, starting with the items UID
	TargetUid targetUid(curItem->GetUniqueId());

	pxr::SdfAttributeSpec::New(targetSpec,
							   _tokens->defnUidAttr,
							   pxr::SdfValueTypeNames->String)
		->SetDefaultValue(VtValue(targetUid._definitionUid.GetString()));

	if (!permHasher.IsEmpty())
	{
		pxr::SdfPath contentSpecPath = curTarget->GetPath().AppendChild(_tokens->content);
		const pxr::SdfPrimSpecHandle contentSpec = _CreatePrimInDataLayer(contentSpecPath);
		pxr::SdfAttributeSpec::New(contentSpec,
								   _tokens->pidHashEntriesAttr,
								   pxr::SdfValueTypeNames->StringArray)
			->SetDefaultValue(VtValue(
			    VtStringArray(permHasher.GetEntries().begin(),
					          permHasher.GetEntries().end())));

		DEBUG_TARGET(
			"Generating permutation id from (%zu) elements ...\n"
			, permHasher.GetEntries().size()
		);

		for (const std::string& hashEntry : permHasher.GetEntries()) {
			DEBUG_TARGET(" - %s\n", hashEntry.c_str());
		}

		// Write the permutation as 12 digit hex
		targetUid._permutationUid = TfToken {
			TfStringPrintf("%012" PRIx64,
				static_cast<std::uint64_t>(permHasher.GetHash64() & 0xFFFFFFFFFFFFULL))
		};

		// Write the permutation uid attribute
		pxr::SdfAttributeSpec::New(targetSpec,
								   _tokens->pidUidAttr,
								   pxr::SdfValueTypeNames->String)
			->SetDefaultValue(VtValue(targetUid._permutationUid.GetString()));
	}

	// TODO: - a more robust encoding mechanism for the prim name
	const TfToken sanitizedTargetUid = TargetUid::ConvertToUsdPrimName(targetUid);
	const pxr::SdfPath targetPrimPath = _targetsRootPath.AppendChild(sanitizedTargetUid);

	bool writeTarget = true;
	if (isConsumingDescendants &&
		curScope->GetConsumerDepth() != curItem->GetDepth())
	{
		// If there's an ancestor permutation that will be consuming this target,
		// rather than generate it we simply add the relevant content to the
		// ancestors scope (currently not implemented)
		//
		// ItemScope* rootScope = rootItem->GetCurrentScope();
		// rootScope->PromoteContent(curScope);
		//
		// writeTarget = false;

		TF_WARN(
			"Ignoring consumable descendant state for asset (%s) at namespace "
			"location <%s> - consumable targets are not currently implemented.\n"
			, curItem->GetUniqueId().c_str()
			, curScope->GetPath().GetText()
		);
	}
	else if (_dataLayer->HasSpec(targetPrimPath))
	{
		DEBUG_TARGET("Elided pre-existing target <%s>)\n", targetPrimPath.GetText());
		writeTarget = false;
	}
	else
	{
		DEBUG_TARGET("Created target <%s>\n", targetPrimPath.GetText());
	}

	auto namespaceEdit = pxr::SdfBatchNamespaceEdit();
	namespaceEdit.Add(targetSpec->GetPath(),
					  writeTarget ? targetPrimPath : pxr::SdfPath());

	if (!TF_VERIFY(_dataLayer->Apply(namespaceEdit),
		"Failed to apply namespace edit for target (%s)"
		, pxr::TfStringify(targetUid).c_str()))
	{
		return std::nullopt;
	}

	// Now that we have a valid target path, add ourself as a
	// dependency to the parent scope.
	if (Item* parentItem = curItem->GetParentItem())
	{
		parentItem->GetCurrentScope()->AddDependency(targetPrimPath);
	}

	// If we have a target, broadcast its existence.
	if (_dataLayer->HasSpec(targetPrimPath))
	{
		return std::optional<TargetUid>(targetUid);
	}

	return std::nullopt;
}

TargetDataRefPtr
LayerWriter::BuildTargetData(const TargetUid& targetUid) const
{
	if (!targetUid.IsValid())
	{
		return{};
	}

	const TfToken targetSpecName = TargetUid::ConvertToUsdPrimName(targetUid);
	const pxr::SdfPath targetSpecPath = _targetsRootPath.AppendChild(targetSpecName);
	const pxr::SdfPrimSpecHandle targetSpec = _GetPrimAtPath(targetSpecPath);

	if (!targetSpec)
	{
		TF_WARN(
			"Invalid request for nonexistent target (%s)."
			, TfStringify(targetUid).c_str()
		);
		return {};
	}

	// Open the data layer stage masked with the target spec prim path and
	// any targets this target depends on.
	pxr::UsdStagePopulationMask mask{_GetDependsOnPaths(targetSpec)};
	mask.Add(targetSpecPath);
	pxr::UsdStageRefPtr stage = pxr::UsdStage::OpenMasked(_dataLayer, mask);
	if (!TF_VERIFY(stage))
	{
		return {};
	}

	TargetDataRefPtr targetData = std::shared_ptr<TargetData>(new TargetData{targetUid});

	pxr::SdfLayerRefPtr permOverlayLayer = pxr::SdfLayer::CreateAnonymous();

	pxr::UsdPrim curDefn = stage->GetPrimAtPath(targetSpecPath);
	if (!TF_VERIFY(curDefn))
	{
		return {};
	}

	targetData->_dependencies = _GetTargetDependencies(curDefn);

	pxr::UsdPrim contentPrim = curDefn.GetChild((_tokens->content));
	targetData->_encapsulatedDefinitions
		= _GetStringArrayAttrAsPathSet(contentPrim, _tokens->encapsulatedPathsAttr);
	targetData->_unencapsulatedDefinitions
		= _GetStringArrayAttrAsPathSet(contentPrim, _tokens->unencapsulatedPathsAttr);

	while (curDefn)
	{
		pxr::UsdPrim infoPrim = curDefn.GetChild((_tokens->info));
		if (!TF_VERIFY(infoPrim))
		{
			return {};
		}

		const std::string uid
			= _GetAttrValue<std::string>(infoPrim, _tokens->uidAttr);
		const std::string scenePath
			= _GetAttrValue<std::string>(infoPrim, _tokens->scenePathAttr);
		if (!TF_VERIFY(!uid.empty() && pxr::SdfPath::IsValidPathString(scenePath)))
		{
			return {};
		}

		PermutationOpVector ops = _ExtractOpsFromDefn(curDefn);
		targetData->_definitionEntries.emplace_back(
			TargetDefinitionEntry{uid, pxr::SdfPath(scenePath), std::move(ops)});

		_ApplyOpsToLayer(curDefn, permOverlayLayer);

		// The next asset (if any) will be the first and only child
		// of the "assets" prim.
		pxr::UsdPrim assetsPrim = curDefn.GetChild((_tokens->assets));
		if (!assetsPrim)
		{
			break;
		}

		auto firstChild = [&assetsPrim]() -> pxr::UsdPrim {
			auto children = assetsPrim.GetFilteredChildren(UsdPrimIsActive);
			auto itr = children.begin();
			if (itr == children.end())
			{
				return {};
			}
			return *itr;
		};

		if (UsdPrim nextDefn = firstChild())
		{
			curDefn = nextDefn;
		}
		else
		{
			break;
		}
	}

	if (!permOverlayLayer->IsEmpty())
	{
		targetData->_permOverlayLayer = permOverlayLayer;
	}

	return targetData;
}


void
LayerWriter::_ConfigureTargetOpsForLevel(
	const pxr::TfSpan<Item*>& itemStack,
	const pxr::SdfPrimSpecHandle& curTarget,
	const size_t itemIdx,
	const size_t scopeIdx,
	const types::int32 overrideDepth,
	PersistentHasher& permHasher)
{
	const Item* thisItem = itemStack[itemIdx];
	const Item* rootItem = itemStack[overrideDepth];
	const pxr::SdfSpecHandle thisScopeSpec
		= _GetPrimAtPath(thisItem->GetScope(scopeIdx)->GetHandle());

	DEBUG_TARGET("`- Spec %zu:%zu <%s> <%s>\n"
		, itemIdx+1
		, scopeIdx
		, thisScopeSpec->GetPath().GetText()
		, thisItem->GetScope(scopeIdx)->GetPath().GetText()
	);

	if (static_cast<types::int32>(itemIdx) < overrideDepth)
	{
		DEBUG_TARGET("Elided non-contributing item (%zu)\n", itemIdx);
		return;
	}

	pxr::SdfVariantSpecHandle variantSpec
		= pxr::SdfSpecStatic_cast<pxr::SdfVariantSpecHandle>(thisScopeSpec);
	const std::string variantSetName = variantSpec->GetOwner()->GetName();
	const std::string variantName = variantSpec->GetName();
	curTarget->SetVariantSelection(variantSetName, variantName);

	permHasher.AddString(variantName);

	// Update the scene path on each permutation op

	const pxr::SdfPath targetOpsSpecPath
		= curTarget->GetPath().AppendChild(_tokens->ops);

	const pxr::SdfPrimSpecHandle targetOpsSpec
		= _CreatePrimInDataLayer(targetOpsSpecPath);

	const pxr::SdfPrimSpecHandle opsHolderSpec
		= util::GetChildSpec(variantSpec->GetPrimSpec(), _tokens->ops);
	if (!TF_VERIFY(opsHolderSpec))
	{
		return;
	}

	const pxr::SdfChildrenView children = opsHolderSpec->GetNameChildren();
	for (const pxr::SdfPrimSpecHandle& opSpec : children)
	{
		const pxr::SdfPath scopeSubPath
			= thisItem->GetScope(scopeIdx)->GetPath()
				  .MakeRelativePath(thisItem->GetPath());

		const pxr::SdfPath targetOpSpecPath
			= targetOpsSpec->GetPath()
				  .AppendChild(TfToken(opSpec->GetName()));

		const pxr::SdfPrimSpecHandle targetOpSpec
			= _CreatePrimInDataLayer(targetOpSpecPath);

		const pxr::SdfPath targetScenePath
			= static_cast<types::int32>(itemIdx) > overrideDepth
			? thisItem->GetPath().ReplacePrefix(
				  rootItem->GetPath(),
				  rootItem->GetPathAtIntroduction())
					  .AppendPath(scopeSubPath)
			: thisItem->GetPathAtIntroduction()
				  .AppendPath(scopeSubPath);

		pxr::SdfAttributeSpec::New(
			targetOpSpec,
			_tokens->scenePathAttr,
			pxr::SdfValueTypeNames->String
		) ->SetDefaultValue(VtValue(targetScenePath.GetString()));
	}
}

void
LayerWriter::_ConfigureTarget(
	const pxr::TfSpan<Item*>& itemStack,
	const pxr::SdfPrimSpecHandle& curTarget,
	const size_t itemIdx,
	const types::int32 overrideDepth,
	PersistentHasher& permHasher)
{
	const Item* thisItem = itemStack[itemIdx];
	const Item* rootItem = itemStack[overrideDepth];
	const Item* curItem = itemStack.back();
	const ItemScope* curScope = thisItem->GetCurrentScope();

	// Write the path at introduction and adjusted scene path into the info spec
	const pxr::SdfPath infoSpecPath = curTarget->GetPath().AppendChild(_tokens->info);
	const pxr::SdfPrimSpecHandle infoSpec = _CreatePrimInDataLayer(infoSpecPath);

	// path at introduction
	pxr::SdfAttributeSpec::New(infoSpec,
		_tokens->pathAtIntroductionAttr,
		pxr::SdfValueTypeNames->String)
		->SetDefaultValue(VtValue(thisItem->GetPathAtIntroduction().GetString()));

	// Write the scene path relative to the ancestor that is providing
	// the override (if any). For example if our input scene path is
	// /world/g/A/s/C
	// and C has no overrides, /C would be the scene path, however if
	// A is providing overrides it would instead be /A/s/C and so on.

	const pxr::SdfPath scopeSubPath
		= curScope->GetPath().MakeRelativePath(thisItem->GetPath());

	const pxr::SdfPath scenePath
		= static_cast<types::int32>(itemIdx) > overrideDepth
		? thisItem->GetPath().ReplacePrefix(
			  rootItem->GetPath(),
			  rootItem->GetPathAtIntroduction())
				  .AppendPath(scopeSubPath)
		: thisItem->GetPathAtIntroduction()
			  .AppendPath(scopeSubPath);

	pxr::SdfAttributeSpec::New(infoSpec, _tokens->scenePathAttr,
							   pxr::SdfValueTypeNames->String)
		->SetDefaultValue(VtValue(scenePath.GetString()));

	// Create the targets content prim. This holds lists of paths of specific
	// categories. Primarily the boundable, material and shader categories

	pxr::SdfPath contentSpecPath = curTarget->GetPath().AppendChild(_tokens->content);
	const pxr::SdfPrimSpecHandle contentSpec = _CreatePrimInDataLayer(contentSpecPath);

	// Transform helper to generate a remapped path as a string.
	auto _RemapToString = [&curItem](const pxr::SdfPath& path)
	{
		return path.ReplacePrefix(
			            curItem->GetPath(),
				        curItem->GetPathAtIntroduction()
				    ).GetString();
	};

	// Transform helper to generate a path as a string.
	auto _PathToString = [](const pxr::SdfPath& path)
	{
		return path.GetString();
	};

	// Helper to write a list of paths to a usd attribute.
	auto _WritePathsToAttr = [&](const pxr::TfSpan<const pxr::SdfPath>& paths,
		                         const pxr::TfToken& attrName,
		                         auto&& transform)
	{
		VtStringArray array;
		array.reserve(paths.size());

		std::transform(paths.cbegin(), paths.cend(),
			           std::back_inserter(array),
			           transform);

		pxr::SdfAttributeSpec::New(contentSpec,
			attrName,
			pxr::SdfValueTypeNames->StringArray)
			    ->SetDefaultValue(VtValue(array));
	};


	bool hasContent = false;

	// Write the content category data - counts and paths. Note that paths are
	// optional (useful for debugging, primarily), and will only get stored if
	// the USDPREGEN_STORE_CONTENT_PATHS is not set to '0'

	for (const pxr::TfToken& category : curScope->GetContentCategoryNames())
	{
		const ItemScope::CategoryView view = curScope->GetCategoryView(category);

		// Invalid category names currently add a map entry to the scope and
		// then never increment the counter, so we must guard here.
		if (view.count <= 0)
		{
			continue;
		}

		hasContent = true;

		const std::string prefix =
			TfStringPrintf("pregen:content:%s:", category.GetText());

		// Count attribute is mandatory

		const pxr::TfToken countAttrName(prefix + "count");

		pxr::SdfAttributeSpec::New(contentSpec,
			countAttrName,
			pxr::SdfValueTypeNames->Int)
			    ->SetDefaultValue(VtValue(view.count));

		if (!view.paths.empty())
		{
			const pxr::TfToken pathsAttrName(prefix + "paths");
			_WritePathsToAttr(view.paths, pathsAttrName, _RemapToString);
		}
	}

	_WritePathsToAttr(curScope->GetOverridePaths(),
		              _tokens->overridesAttr,
		              _PathToString);

	// Deactivate the content spec
	contentSpec->SetActive(hasContent);
}


SdfPathVector
LayerWriter::_GetDependsOnPaths(const pxr::SdfPrimSpecHandle& targetSpec) const
{
	pxr::SdfPathVector depPaths;

	const pxr::SdfPrimSpecHandle targetInfo
		= _GetPrimAtPath(targetSpec->GetPath().AppendChild(_tokens->info));

	if (!TF_VERIFY(targetInfo))
	{
		return {};
	}

	const pxr::SdfPath relPath
		= targetInfo->GetPath().AppendProperty(_tokens->dependsOnRel);
	if (pxr::SdfRelationshipSpecHandle relSpec
		    = _dataLayer->GetRelationshipAtPath(relPath))
	{
		const pxr::SdfTargetsProxy targets = relSpec->GetTargetPathList();
		const pxr::SdfListProxy appendedItems = targets.GetAppendedItems();
		depPaths.reserve(appendedItems.size());
		for (const SdfPath& path : appendedItems) {
			depPaths.push_back(path);
		}
	}

	return depPaths;
}

std::vector<TargetUid>
LayerWriter::_GetTargetDependencies(const pxr::UsdPrim& target) const
{
	pxr::UsdPrim infoPrim = target.GetChild((_tokens->info));
	if (!TF_VERIFY(infoPrim))
	{
		return {};
	}

	std::vector<TargetUid> deps;

	if (pxr::UsdRelationship rel = infoPrim.GetRelationship(_tokens->dependsOnRel))
	{
		if (pxr::SdfPathVector targets; rel.GetTargets(&targets))
		{
			deps.reserve(targets.size());
			for (const pxr::SdfPath& targetPath : targets)
			{
				if (UsdPrim targetPrim = target.GetStage()->GetPrimAtPath(targetPath))
				{
					TargetUid dep;
					dep._definitionUid = TfToken(_GetAttrValue<std::string>(
											 targetPrim,
											 _tokens->defnUidAttr));
					dep._permutationUid =  TfToken(_GetAttrValue<std::string>(
											 targetPrim,
											 _tokens->pidUidAttr,
											 /*emptyOk=*/true));
					if (TF_VERIFY(dep))
					{
						deps.push_back(dep);
					}
				}
			}
		}
	}

	return deps;
}

} // namespace internal

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
