// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"
#include "UsdPregen/types.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/usd/sdf/layer.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/stage.h"
#include "USDIncludesEnd.h"

#include <memory>
#include <optional>
#include <string>
#include <variant>

PXR_NAMESPACE_OPEN_SCOPE

	class SdfPrimSpec;
	class SdfSpec;
	class SdfLayer;

	template <typename T>
	class SdfHandle;

	template <typename T>
	class TfRefPtr;

	using SdfPrimSpecHandle = SdfHandle<SdfPrimSpec>;
	using SdfSpecHandle = SdfHandle<SdfSpec>;

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

class ExtAssetDefinition;
class TargetUid;

using TargetDataRefPtr = std::shared_ptr<class TargetData>;
using PrimPermutationConstRefPtr = std::shared_ptr<const class PrimPermutation>;

namespace internal
{

class Item;
class PersistentHasher;
struct ScopeHandle;

/// \class LayerWriter
///
/// Writes tracker discovery results into an intermediate USD layer.
///
/// LayerWriter is responsible for serializing the results of discovery and
/// traversal into a structured SdfLayer. This layer acts as an intermediate
/// representation of:
///   - External asset definitions
///   - Variant permutations and their associated ops
///   - Generated targets and their dependency relationships
///   - Scene paths encountered during traversal
///
/// The writer is driven incrementally by the tracker. Most public methods
/// correspond to events that occur during discovery (e.g. an asset definition
/// being encountered, a permutation being applied, etc).
///
/// The resulting layer can later be queried to reconstruct TargetData,
/// including dependency graphs and permutation overlay layers.
///
/// Internally, the data is organized under a few well-known root paths:
///   /__definitions__  - External asset definitions
///   /__targets__      - Generated targets
///   /__scene__        - Scene structure encountered during traversal
///                       (for diagnostic purposes only)
///
/// The main reason this data is authored to USD and not a custom data structure
/// is to assist with debugging and introspection. When a large scene generates
/// many targets, it is easier and more intuitive to navigate the definition and
/// target data using UsdView than a debugger. In the future we'll add
/// representative bounding boxes to the "scene" prims with relationships that
/// link back to the associated targets.
///
class LayerWriter
{
public:
	LayerWriter(const pxr::UsdStageRefPtr& stage);

	~LayerWriter() = default;

	/// Record that an external asset definition has been discovered.
	///
	/// Creates (or reuses) an "AssetDefinition" prim spec under
	/// /__definitions__ and writes identifying information (uid, name,
	/// identifier, version,  metadata) as properties.
	///
	/// \return A handle to the definition scope in the data layer.
	ScopeHandle AssetDefinitionAdded(const ExtAssetDefinition& assetDefn);

	/// Record a permutation applied at a given nesting level.
	///
	/// Creates or updates a variant set spec on the current scope and
	/// serializes the permutation's ops into the layer.
	///
	/// \return A handle to the created variant spec.
	ScopeHandle AddPermutation(
		const Item* curItem,
		const pxr::SdfPath& primPath,
		const PrimPermutationConstRefPtr& perm,
		types::int32 nestingLevel);

	/// Record a dependency between a child asset and its parent.
	///
	/// This authors an inherits arc under the parent’s "assets" scope so that
	/// dependency relationships can later be reconstructed.
	void AddChildAssetDependencyToParent(const Item* curItem);

	/// Generate (or reuse) a target prim spec for the current item stack under
	/// /__targets__
	///
	/// The created target spec inherits the root definition spec needed by the
	/// target, and configures the definitions permutation variants (if any).
	///
	/// A target represents a fully resolved combination of:
	///   - Asset definition
	///   - Permutations
	///   - Override paths
	///
	/// Targets are uniquely identified via a hash derived from these inputs.
	/// If an equivalent target already exists, it will be reused.
	///
	/// \return The generated TargetUid, or nullopt if no target was produced.
	std::optional<TargetUid> GenerateOrReuseTarget(const pxr::TfSpan<Item*>& itemStack);

	/// Reconstruct TargetData for a previously generated target.
	///
	/// This opens a masked stage over the internal data layer and extracts:
	///   - Dependency information
	///   - Definition paths
	///   - Target info chain
	///   - A permutation overlay layer describing applied ops
	///
	/// \return A populated TargetData object, or null if invalid.
	TargetDataRefPtr BuildTargetData(const TargetUid& targetUid) const;

	/// Record that a scene path was encountered during traversal.
	///
	/// This builds a structural representation of the scene under
	/// /__scene__, preserving the prim type information from the source stage
	/// when available. This data is only used for diagnostic purposes.
	void AddNewScenePath(const pxr::SdfPath& path);

	/// Export the internal data layer to disk.
	///
	/// \return True on success.
	bool Save(const std::string& filename);

private:

	pxr::SdfPrimSpecHandle _CreatePrimInDataLayer(const pxr::SdfPath& primPath);

	pxr::SdfPrimSpecHandle _GetAssetsContainerSpec(
	    const pxr::SdfSpecHandle& parentSpec);

	pxr::SdfSpecHandle _GetPrimAtPath(const ScopeHandle& handle);

	pxr::SdfPrimSpecHandle _GetPrimAtPath(const pxr::SdfPath& primPath) const;

	void _ConfigureTargetOpsForLevel(
		const pxr::TfSpan<Item*>& itemStack,
		const pxr::SdfPrimSpecHandle& curTarget,
		const size_t itemIdx,
		const size_t scopeIdx,
		const types::int32 overrideDepth,
		PersistentHasher& permHasher);

	void _ConfigureTarget(
		const pxr::TfSpan<Item*>& itemStack,
		const pxr::SdfPrimSpecHandle& curTarget,
		const size_t itemIdx,
		const types::int32 overrideDepth,
		PersistentHasher& permHasher);

	pxr::SdfPathVector _GetDependsOnPaths(
		const pxr::SdfPrimSpecHandle& targetSpec) const;

	std::vector<TargetUid> _GetTargetDependencies(
		const pxr::UsdPrim& target) const;

	pxr::SdfLayerRefPtr _dataLayer;

	pxr::UsdStageRefPtr _stage;
};

} // namespace internal

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
