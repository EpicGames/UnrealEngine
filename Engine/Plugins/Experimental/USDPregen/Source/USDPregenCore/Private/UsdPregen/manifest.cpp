// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/manifest.h"

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PREGEN_NAMESPACE_OPEN_SCOPE

TargetUid
Manifest::GetTargetUid() const
{
	return _targetData ? _targetData->GetUniqueId() : TargetUid{};
}

void
Manifest::AddProduct(const Product& product)
{
	// TODO validation, duplicates etc.
	_products.push_back(product);
}

const std::vector<Product>&
Manifest::GetProducts() const
{
	return _products;
}

void
Manifest::SetTargetData(const TargetDataRefPtr& targetData)
{
	_targetData = targetData;
}

const TargetDataRefPtr&
Manifest::GetTargetData() const
{
	return _targetData;
}

bool
Manifest::IsValid() const
{
	return _targetData && _targetData->IsValid() && !_products.empty();
}

Manifest::operator bool() const noexcept
{
	return IsValid();
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
