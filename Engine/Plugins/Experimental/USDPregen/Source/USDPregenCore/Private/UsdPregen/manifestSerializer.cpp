// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/manifestSerializer.h"

PREGEN_NAMESPACE_OPEN_SCOPE

// virtual
ManifestSerializer::~ManifestSerializer() = default;

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
