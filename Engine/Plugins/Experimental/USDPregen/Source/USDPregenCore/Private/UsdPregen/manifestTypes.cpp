// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/manifestTypes.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/enum.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/registryManager.h"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

PXR_NAMESPACE_OPEN_SCOPE

TF_REGISTRY_FUNCTION(TfEnum)
{
	TF_ADD_ENUM_NAME(ManifestLoadStatus::Loaded);
	TF_ADD_ENUM_NAME(ManifestLoadStatus::DoesNotExist);
	TF_ADD_ENUM_NAME(ManifestLoadStatus::Error);

	TF_ADD_ENUM_NAME(ManifestSaveStatus::Saved);
	TF_ADD_ENUM_NAME(ManifestSaveStatus::NotSaved);
	TF_ADD_ENUM_NAME(ManifestSaveStatus::Error);
}

PXR_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
