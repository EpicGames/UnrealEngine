// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/permutationOps.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/enum.h"
#include "pxr/base/tf/staticTokens.h"
#include "pxr/usd/sdf/attributeSpec.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/sdf/primSpec.h"
#include "pxr/usd/sdf/valueTypeName.h"
#include "pxr/usd/usd/inherits.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/variantSets.h"
#include "USDIncludesEnd.h"

#include <set>
#include <string>
#include <memory>
#include <unordered_map>
#include <utility>
#include <vector>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE // for TF macros

// clang-format off
TF_DEFINE_PRIVATE_TOKENS(
	_tokens,
	((variantOpPrefix, "variantset_"))
	((inheritOpPrefix, "inherit_"))
);
// clang-format on

PREGEN_NAMESPACE_OPEN_SCOPE

namespace
{

std::unordered_map<std::string, PermutationOp::FromSpecFn>&
_GetFactoryRegistry()
{
	static std::unordered_map<std::string, PermutationOp::FromSpecFn> registry;
	return registry;
}

// Reads a string-typed opargs:* default value from a prim spec.
std::string
_GetOpargString(const pxr::SdfPrimSpecHandle& primSpec, const char* attrName)
{
	if (!primSpec)
	{
		return {};
	}

	const pxr::SdfAttributeSpecHandle attr = primSpec->GetAttributes()[pxr::TfToken(attrName)];
	if (!attr)
	{
		return {};
	}

	const pxr::VtValue value = attr->GetDefaultValue();
	if (!value.IsHolding<std::string>())
	{
		return {};
	}

	return value.UncheckedGet<std::string>();
}

} // anonymous namespace

// virtual
PermutationOp::~PermutationOp()
{
}

// static
bool
PermutationOp::RegisterFactory(const std::string& typeName, FromSpecFn fn)
{
	if (typeName.empty() || !fn)
	{
		return false;
	}

	auto& registry = _GetFactoryRegistry();
	const auto [it, inserted] = registry.emplace(typeName, std::move(fn));
	return inserted;
}

// static
PermutationOpRefPtr
PermutationOp::CreateFromSpec(const pxr::SdfPrimSpecHandle& primSpec)
{
	if (!primSpec)
	{
		return nullptr;
	}

	const std::string typeName = primSpec->GetTypeName();
	const auto& registry = _GetFactoryRegistry();
	const auto it = registry.find(typeName);
	if (it == registry.end())
	{
		TF_WARN(
			"No PermutationOp factory registered for type name '%s'",
			typeName.c_str()
		);
		return nullptr;
	}

	return it->second(primSpec);
}

UsdVariantPermutationOp::UsdVariantPermutationOp(
	const std::string& variantSetName,
	const std::string& variantName)
	: _variantSetName(variantSetName)
	, _variantName(variantName)
{
}

// static
PermutationOpRefPtr
UsdVariantPermutationOp::FromSerialized(const std::string& variantSetName,
                                        const std::string& variantName)
{
	return std::shared_ptr<UsdVariantPermutationOp>(
		new UsdVariantPermutationOp(variantSetName, variantName)
	);
}

// virtual
void
UsdVariantPermutationOp::Apply(pxr::UsdPrim& prim)
{
	DEBUG_PERMUTATION(
		"Applying permutation op (UsdVariant) {%s=%s} on prim <%s>\n"
		,_variantSetName.c_str(), _variantName.c_str()
		, prim.GetPrimPath().GetText()
	);

	pxr::UsdVariantSets variantSets = prim.GetVariantSets();
	if (!variantSets[_variantSetName].SetVariantSelection(_variantName))
	{
		TF_WARN("failed to set variant {%s=%s} on prim <%s> - one "
			"or more permutations may be misconfigured."
			,_variantSetName.c_str(), _variantName.c_str()
			,prim.GetPrimPath().GetText());
	}
};

// virtual
std::string
UsdVariantPermutationOp::GetUniqueId() const
{
	return _tokens->variantOpPrefix.GetString()
			   + _variantSetName + "_" + _variantName;
}

// virtual
void
UsdVariantPermutationOp::Serialize(pxr::SdfPrimSpecHandle& primSpec) const
{
	primSpec->SetTypeName(std::string("UsdVariantSelectionOp"));

	auto _WriteStringAttr = [&primSpec] (
	  const std::string& attrName, const std::string& attrValue
	)
	{
		pxr::SdfAttributeSpec::New(primSpec, attrName, pxr::SdfValueTypeNames->String)
		  ->SetDefaultValue(VtValue(attrValue));
	};

	_WriteStringAttr("opargs:variantSet", _variantSetName);
	_WriteStringAttr("opargs:variant", _variantName);
}

std::pair<std::string, std::string>
UsdVariantPermutationOp::GetVariantSelection() const
{
	return std::make_pair(_variantSetName, _variantName);
}

bool
UsdVariantPermutationOp::operator==(const UsdVariantPermutationOp& rhs) const
{
	return
		   _variantSetName == rhs._variantSetName
		&& _variantName == rhs._variantName;
}

bool
UsdVariantPermutationOp::operator!=(const UsdVariantPermutationOp& rhs) const
{
	return !(*this == rhs);
}


UsdInheritPermutationOp::UsdInheritPermutationOp(
	const pxr::SdfPath& pathToInherit,
	pxr::SdfListOpType listOpType)
	: _pathToInherit(pathToInherit)
	, _listOpType(listOpType)
{
}

// static
PermutationOpRefPtr
UsdInheritPermutationOp::FromSerialized(const pxr::SdfPath& pathToInherit,
                                         pxr::SdfListOpType listOpType)
{
	return std::shared_ptr<UsdInheritPermutationOp>(
		new UsdInheritPermutationOp(pathToInherit, listOpType)
	);
}

// virtual
void
UsdInheritPermutationOp::Apply(pxr::UsdPrim& prim)
{
	DEBUG_PERMUTATION(
		"Applying permutation op (UsdInherit) <%s> on prim <%s>\n"
		, _pathToInherit.GetText(), prim.GetPrimPath().GetText()
	);

	if(!prim.GetInherits().AddInherit(_pathToInherit))
	{
		TF_WARN("failed to add inherit <%s> on prim <%s> - one "
			"or more permutations may be misconfigured."
			, _pathToInherit.GetText(), prim.GetPrimPath().GetText()
		);
	}
};

// virtual
std::string
UsdInheritPermutationOp::GetUniqueId() const
{
	std::string uid = _tokens->inheritOpPrefix.GetString() + _pathToInherit.GetName();
	// if this is a root class we're inheriting, just the name will suffice
	if (_pathToInherit.GetPathElementCount() == 1)
	{
		return uid;
	}

	// otherwise we need a hash of the full path to guarantee uniqueness
	return uid + "_" + std::to_string(TfHash()(_pathToInherit));
}

// virtual
void
UsdInheritPermutationOp::Serialize(pxr::SdfPrimSpecHandle& primSpec) const
{
	primSpec->SetTypeName(std::string("UsdInheritOp"));

	auto _WriteStringAttr = [&primSpec] (
	  const std::string& attrName, const std::string& attrValue
	)
	{
		pxr::SdfAttributeSpec::New(primSpec, attrName, pxr::SdfValueTypeNames->String)
			->SetDefaultValue(VtValue(attrValue));
	};

	_WriteStringAttr("opargs:pathToInherit", _pathToInherit.GetString());
	_WriteStringAttr("opargs:listOpType", pxr::TfEnum::GetName(_listOpType));
}

const pxr::SdfPath&
UsdInheritPermutationOp::GetPathToInherit() const
{
	return _pathToInherit;
}

pxr::SdfListOpType
UsdInheritPermutationOp::GetListOpType() const
{
	return _listOpType;
}

SchemaApplyPermutationOp::SchemaApplyPermutationOp(
	const std::string& schemaName)
	: _schemaName(schemaName)
{
}

// static
PermutationOpRefPtr
SchemaApplyPermutationOp::FromSerialized(const std::string& schemaName)
{
	return std::make_shared<SchemaApplyPermutationOp>(schemaName);
}

// virtual
void
SchemaApplyPermutationOp::Apply(pxr::UsdPrim& prim)
{
	return;
}

// virtual
std::string
SchemaApplyPermutationOp::GetUniqueId() const
{
	return "schemaapply_" + _schemaName;
}

std::string
SchemaApplyPermutationOp::GetSchemaName() const
{
	return _schemaName;
}

// virtual
void
SchemaApplyPermutationOp::Serialize(pxr::SdfPrimSpecHandle& primSpec) const
{
	primSpec->SetTypeName(std::string("UsdSchemaApplyOp"));

	auto _WriteStringAttr = [&primSpec] (
	  const std::string& attrName, const std::string& attrValue
	)
	{
		pxr::SdfAttributeSpec::New(primSpec, attrName, pxr::SdfValueTypeNames->String)
			->SetDefaultValue(VtValue(attrValue));
	};

	_WriteStringAttr("opargs:schemaName", _schemaName);
}

// Register factories for the built-in op types. Type names must match the
// strings written by each subclass's Serialize() implementation above.
namespace
{

const bool _builtinFactoriesRegistered = []()
{
	PermutationOp::RegisterFactory(
		"UsdVariantSelectionOp",
		[](const pxr::SdfPrimSpecHandle& spec) -> PermutationOpRefPtr
		{
			const std::string variantSet
				= _GetOpargString(spec, "opargs:variantSet");
			const std::string variant
				= _GetOpargString(spec, "opargs:variant");
			if (variantSet.empty() || variant.empty())
			{
				TF_WARN("UsdVariantSelectionOp factory: missing required opargs");
				return nullptr;
			}
			return UsdVariantPermutationOp::FromSerialized(variantSet, variant);
		}
	);

	PermutationOp::RegisterFactory(
		"UsdInheritOp",
		[](const pxr::SdfPrimSpecHandle& spec) -> PermutationOpRefPtr
		{
			const std::string pathToInheritStr
				= _GetOpargString(spec, "opargs:pathToInherit");
			if (!pxr::SdfPath::IsValidPathString(pathToInheritStr))
			{
				TF_WARN("UsdInheritOp factory: invalid or missing opargs:pathToInherit");
				return nullptr;
			}

			const std::string listOpTypeStr
				= _GetOpargString(spec, "opargs:listOpType");
			bool found = false;
			pxr::SdfListOpType listOpType
				= pxr::TfEnum::GetValueFromName<pxr::SdfListOpType>(listOpTypeStr, &found);
			if (!found)
			{
				TF_WARN(
					"UsdInheritOp factory: invalid list op type '%s' - "
					"defaulting to 'prepended'",
					listOpTypeStr.c_str()
				);
				listOpType = pxr::SdfListOpTypePrepended;
			}

			return UsdInheritPermutationOp::FromSerialized(
				pxr::SdfPath(pathToInheritStr), listOpType);
		}
	);

	PermutationOp::RegisterFactory(
		"UsdSchemaApplyOp",
		[](const pxr::SdfPrimSpecHandle& spec) -> PermutationOpRefPtr
		{
			const std::string schemaName
				= _GetOpargString(spec, "opargs:schemaName");
			if (schemaName.empty())
			{
				TF_WARN("UsdSchemaApplyOp factory: missing opargs:schemaName");
				return nullptr;
			}
			return SchemaApplyPermutationOp::FromSerialized(schemaName);
		}
	);

	return true;
}();

} // anonymous namespace

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
