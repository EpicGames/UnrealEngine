// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "persistentHasher.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/pathUtils.h"
#include "pxr/base/tf/stringUtils.h"
#include "USDIncludesEnd.h"

#include <cstdint>
#include <string>

PREGEN_NAMESPACE_OPEN_SCOPE

namespace internal
{

std::uint64_t
PersistentHasher::GetHash64() const
{
	return _hash;
}

std::uint32_t
PersistentHasher::GetHash32() const
{
	std::uint64_t x = _hash;
	x ^= x >> 33;
	x *= 0xff51afd7ed558ccdULL;
	x ^= x >> 33;
	x *= 0xc4ceb9fe1a85ec53ULL;
	x ^= x >> 33;
	return static_cast<uint32_t>(x ^ (x >> 32));
}

void
PersistentHasher::AddString(const std::string& str)
{
	_entries.push_back(str);

	std::size_t len = str.size();
	_AddBytes(&len, sizeof(len));
	_AddBytes(str.data(), str.size());
}

void
PersistentHasher::AddFilePath(const std::string& filePath)
{
	const std::string normPath = pxr::TfNormPath(filePath);
	_entries.push_back(normPath);
	AddString(normPath);
}

void
PersistentHasher::AddUint64(std::uint64_t val)
{
	_entries.push_back(pxr::TfStringPrintf("%" PRIu64, val));
	_AddBytes(&val, sizeof(val));
}

const std::vector<std::string>&
PersistentHasher::GetEntries() const &
{
	return _entries;
}

bool
PersistentHasher::IsEmpty() const
{
	return _entries.empty();
}

void
PersistentHasher::_AddBytes(const void* data, size_t size)
{
	const std::uint8_t* bytes = static_cast<const std::uint8_t*>(data);
	for (std::size_t i = 0; i < size; ++i) {
		_hash ^= bytes[i];
		_hash *= _prime;
	}
}

} // namespace internal

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
