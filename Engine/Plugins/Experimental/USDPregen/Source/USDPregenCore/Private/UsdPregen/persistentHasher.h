// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include <cstdint>
#include <string>
#include <vector>

PREGEN_NAMESPACE_OPEN_SCOPE

namespace internal {

class PersistentHasher
{
public:

	std::uint64_t GetHash64() const;

	std::uint32_t GetHash32() const;

	void AddString(const std::string& str);

	void AddFilePath(const std::string& path);

	void AddUint64(const std::uint64_t val);

	const std::vector<std::string>& GetEntries() const &;

	bool IsEmpty() const;

 private:

	void _AddBytes(const void* data, std::size_t size);

	static constexpr std::uint64_t _offset = 14695981039346656037ULL;
	static constexpr std::uint64_t _prime = 1099511628211ULL;
	std::uint64_t _hash = _offset;
	std::vector<std::string> _entries;
};

} // namespace internal

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
