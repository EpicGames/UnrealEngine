// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/primPermutation.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/staticTokens.h"
#include "pxr/usd/sdf/path.h"
#include "USDIncludesEnd.h"

#include <string>
#include <memory>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE // for TF macros

// clang-format off
TF_DEFINE_PRIVATE_TOKENS(
	_tokens,
	((permEmpty, "perm__EMPTY_PERMUTATION__"))
	((permPrefix, "perm__"))
	((permSeparator, "__X__"))
);
// clang-format on

PREGEN_NAMESPACE_OPEN_SCOPE

PrimPermutation::PrimPermutation() = default;

PrimPermutation::PrimPermutation(const pxr::SdfPath& path)
	: _path(path)
{
}

PrimPermutation::PrimPermutation(
	const pxr::SdfPath& path,
	const PermutationOpVector& ops)
	: _path(path)
{
	_ops.reserve(ops.size());
	for (const PermutationOpRefPtr& op : ops)
	{
		AppendOp(op);
	}
}

const pxr::SdfPath&
PrimPermutation::GetPath() const &
{
	return _path;
}

std::string
PrimPermutation::GetUniqueId() const
{
	if (_ops.empty())
	{
		return _tokens->permEmpty.GetString();
	}

	std::string name;
	name.reserve(256);

	name += _tokens->permPrefix.GetString();
	for (size_t i = 0; i < _ops.size(); ++i)
	{
		name += _ops[i]->GetUniqueId();
		if (i + 1 != _ops.size())
		{
			name += _tokens->permSeparator.GetString();
		}
	}

	return name;
}

bool
PrimPermutation::AppendOp(const PermutationOpRefPtr& op)
{
	const std::string uid = op->GetUniqueId();
	if (!_addedOps.count(uid))
	{
		_ops.push_back(op);
		_addedOps.insert(uid);
		return true;
	}

	return false;
}

const PermutationOpVector&
PrimPermutation::GetOps() const &
{
	return _ops;
}

void
PrimPermutation::RemoveOps(const PermutationOpVector& opsToRemove)
{
	for (const PermutationOpRefPtr& op : opsToRemove)
	{
		auto itr = std::find(_ops.begin(), _ops.end(), op);
		if (itr != _ops.end())
		{
			_ops.erase(itr);
			_addedOps.erase(op->GetUniqueId());
		}
	}
}

bool
PrimPermutation::IsEmpty() const
{
	return _ops.empty();
}

bool
PrimPermutation::GetConsumesDescendants() const
{
	return _consumesDescendants;
}

void
PrimPermutation::SetConsumesDescendants(bool value)
{
	_consumesDescendants = value;
}

PrimPermutation::operator bool() const noexcept
{
	return !IsEmpty();
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
