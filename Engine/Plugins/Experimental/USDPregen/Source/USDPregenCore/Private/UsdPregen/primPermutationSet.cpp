// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "primPermutationSet.h"

#include "UsdPregen/primPermutation.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

#include <utility>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE // for TF macros

PREGEN_NAMESPACE_OPEN_SCOPE

namespace internal {

PrimPermutationSet::PrimPermutationSet(const pxr::SdfPath& path)
	: _path(path)
{
}

const pxr::SdfPath&
PrimPermutationSet::GetPath() const &
{
	return _path;
}

void
PrimPermutationSet::AddPermutations(const std::vector<PrimPermutation>& perms)
{
	FinalizeCurrentEntries();
	_curEntries.reserve(perms.size());
	for (const PrimPermutation& perm : perms)
	{
		_AddPermutation(std::make_shared<PrimPermutation>(std::move(perm)));
	}
}

std::vector<PrimPermutationRefPtr>
PrimPermutationSet::GetPermutations() const
{
	// Note this is deliberately returned by value.
	return _curEntries;
}

void
PrimPermutationSet::FinalizeCurrentEntries()
{
	DEBUG_PERMUTATION(
		"Finalizing (%zu) entries for iteration (%zu) on permutation "
		"set for prim <%s>\n"
		, _curEntries.size()
		, _iterationNum
		, _path.GetText()
	);

	// Move the uids of the current entries into the previous entries so that if
	// a discovery plugin adds the same permutation a second time we know to skip it.
	for (const PrimPermutationRefPtr& primPerm : _curEntries)
	{
		_prevEntries.insert(primPerm->GetUniqueId());
	}

	_curEntries.clear();
	_iterationNum++;
}

bool
PrimPermutationSet::IsEmpty() const
{
	return _curEntries.empty();
}

void
PrimPermutationSet::_AddPermutation(const PrimPermutationRefPtr& primPerm)
{
	if (!TF_VERIFY(_path == primPerm->GetPath()))
	{
		return;
	}

	const std::string uid = primPerm->GetUniqueId();

	if (!_prevEntries.count(uid))
	{
		_curEntries.push_back(primPerm);

		DEBUG_PERMUTATION(
			"Added permutation (%s) to prim <%s>\n"
			, uid.c_str()
			, _path.GetText()
		);
	}
	else
	{
		DEBUG_PERMUTATION(
			"Ignoring previously processed permutation (%s)\n",
			uid.c_str()
		);
	}
}

} // namespace internal

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
