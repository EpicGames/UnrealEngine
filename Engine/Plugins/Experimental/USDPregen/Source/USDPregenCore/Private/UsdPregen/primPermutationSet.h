// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/usd/sdf/path.h"
#include "USDIncludesEnd.h"

#include <cstddef> // for size_t
#include <memory>
#include <set>
#include <string>
#include <vector>

PREGEN_NAMESPACE_OPEN_SCOPE

class PrimPermutationRange;

using PrimPermutationRefPtr = std::shared_ptr<class PrimPermutation>;

namespace internal {

/// PrimPermutationSet tracks the set of permutations discovered for a
/// specific prim during traversal.
///
/// Permutations may be discovered incrementally as traversal explores
/// the scene. This class allows the tracker to accumulate newly discovered
/// permutations while avoiding duplicates that were produced in previous
/// iterations.
///
/// Each permutation is associated with a single prim path.
///
/// ---------------------------------------------------------------------------
class PrimPermutationSet
{
public:
	PrimPermutationSet(const pxr::SdfPath& path);

	const pxr::SdfPath& GetPath() const &;

	void AddPermutations(const std::vector<PrimPermutation>& perms);

	std::vector<PrimPermutationRefPtr> GetPermutations() const;

	void FinalizeCurrentEntries();

	bool IsEmpty() const;

private:

	void _AddPermutation(const PrimPermutationRefPtr& primPerm);

	pxr::SdfPath _path;

	std::vector<PrimPermutationRefPtr> _curEntries;

	std::set<std::string> _prevEntries;

	size_t _iterationNum = 0;
};

} // namespace internal

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
