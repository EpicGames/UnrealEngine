// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregen/sceneDiscovery.h"

#include "UsdPregen/discoveryOptions.h"
#include "UsdPregen/target.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/primRange.h"
#include "pxr/usd/usd/stage.h"
#include "USDIncludesEnd.h"

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE

PREGEN_NAMESPACE_OPEN_SCOPE

SceneDiscovery::SceneDiscovery(const pxr::UsdStageRefPtr& stage)
	: SceneDiscovery(stage, {})
{
}

SceneDiscovery::SceneDiscovery(
	const pxr::UsdStageRefPtr& stage,
	const DiscoveryOptions& options)
	: _stage(stage)
	, _tracker(SceneTracker::Create(_stage, options))
{
}

bool
SceneDiscovery::TraverseAndFindTargets(SceneDiscovery::ResultMap& results)
{
	results.clear();

	if (!_stage || !_tracker)
	{
		return false;
	}

	class _ScopedCallback
	{
	public:
		_ScopedCallback(SceneTracker& tracker, SceneTracker::TargetCreatedCallback callback)
			: _trackerRef(tracker)
		{
			_trackerRef.SetTargetCreatedCallback(std::move(callback));
		}

		~_ScopedCallback()
		{
			_trackerRef.RemoveTargetCreatedCallback();
		}

	private:
		SceneTracker& _trackerRef;
	};

	_ScopedCallback ScopedCallback(
		*_tracker,
		[&results](const pxr::SdfPath& path, const TargetUid& targetUid)
		{
			results[path].push_back(targetUid);
		}
	);

	const pxr::SdfPath initialPath = _tracker->GetInitialPath();

	if (initialPath.IsEmpty())
	{
		TF_WARN(
		    "Failed to get initial traversal path for stage @%s@"
			, _stage->GetRootLayer()->GetIdentifier().c_str()
		);
		return false;
	}

	const pxr::UsdPrim initialPrim = _stage->GetPrimAtPath(initialPath);
	if (!initialPrim)
	{
		TF_WARN(
		    "Failed to get valid prim for initial path <%s> on stage @%s@"
		    , initialPath.GetText()
			, _stage->GetRootLayer()->GetIdentifier().c_str()
		);
		return false;
	}

	DEBUG_TRACKING(
	    "Begin traversal from prim <%s>%s\n"
		, initialPrim.GetPrimPath().GetText()
		, initialPrim == _stage->GetDefaultPrim() ? " (stage default prim)" : ""
	);

	_Traverse(initialPrim);

	bool success = !_tracker->HasErrors();
	return success;
}

TargetDataRefPtr
SceneDiscovery::GetTargetData(const TargetUid& targetUid)
{
	return _tracker ? _tracker->GetTargetData(targetUid) : nullptr;
}

bool
SceneDiscovery::SaveDiscoveryData(const std::string& filename)
{
	if (!_stage || !_tracker)
	{
		return false;
	}

	return _tracker->SaveDataLayer(filename);
}

void
SceneDiscovery::_Traverse(const pxr::UsdPrim& prim)
{
	TrackedPrim tracked = _tracker->StartTrackingPrim(prim);

	if (!tracked)
	{
		return;
	}

	if (tracked.HasUnprocessedPermutations())
	{
		while(tracked.PrepareNextPermutation())
		{
			_Traverse(prim);
		}
		return;
	}

	UsdPrimSiblingRange children
		= prim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(
		    _tracker->GetOptions().traversalPredicate));

	//UsdPrimSiblingRange children = prim.GetAllChildren();

	for (pxr::UsdPrim childPrim : children)
	{
		_Traverse(childPrim);
	}
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK

