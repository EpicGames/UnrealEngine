// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/sceneTracker.h"

#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/assetDefinitionRegistry.h"
#include "UsdPregen/discoveryOptions.h"
#include "UsdPregen/discoveryPlugin.h"
#include "UsdPregen/discoveryPluginRegistry.h"
#include "UsdPregen/primPermutation.h"
#include "UsdPregen/target.h"
#include "UsdPregen/types.h"

#include "item.h"
#include "layerWriter.h"
#include "primPermutationSet.h"
#include "util.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/pathUtils.h"
#include "pxr/base/tf/span.h"
#include "pxr/usd/pcp/layerStack.h"
#include "pxr/usd/pcp/primIndex.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/sdf/primSpec.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usd/variantSets.h"
#include "pxr/usd/usdGeom/xform.h"
#include "USDIncludesEnd.h"

#include <memory>
#include <optional>
#include <set>
#include <sstream>
#include <string>
#include <vector>
#include <unordered_map>
#include <utility>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE

PREGEN_NAMESPACE_OPEN_SCOPE

using namespace internal;

TrackedPrim::TrackedPrim()
{
}

TrackedPrim::TrackedPrim(const pxr::UsdPrim& prim, std::shared_ptr<SceneTracker> tracker)
	: _prim(prim)
	, _tracker(tracker)
{
	// Note: this constructor is only available to SceneTracker, via StartTrackingPrim()
	if (prim && tracker)
	{
		switch (tracker->GetOptions().discoveryMode)
		{
		case DiscoveryMode::AllPermutations:
			_phase = _Phase::Started;
			break;
		case DiscoveryMode::ComposedPermutationOnly:
			_phase = _Phase::Completed;
			break;
		default:
			TF_CODING_ERROR("Unsupported discovery mode.");
			break;
		}
	}
}

TrackedPrim::~TrackedPrim()
{
	// Note: We don't check for _prim.IsValid() here. Even if the underlying USD
	// prim has expired, the path remains valid, and we must pop the tracker
	// stack to maintain hierarchy integrity. The phase check effectively
	// prevents double-popping from moved-from objects.
	if (_phase != _Phase::Invalid)
	{
		if (SceneTrackerRefPtr tracker = _tracker.lock())
		{
			tracker->_PopAssetOrPermutation(_prim.GetPrimPath());
			tracker->_PopPurpose();
		}
	}
}

TrackedPrim::TrackedPrim(TrackedPrim&& other) noexcept
	: _prim(other._prim)
	, _tracker(std::move(other._tracker))
	, _permState(std::move(other._permState))
	, _phase(other._phase)
{
	// Ensure 'other' no longer thinks it owns a stack push.
	other._prim = pxr::UsdPrim();
	other._phase = _Phase::Invalid;
	other._tracker.reset();
}

bool
TrackedPrim::HasUnprocessedPermutations() const
{
	if (_phase == _Phase::Invalid || _phase == _Phase::Completed)
	{
		return false;
	}

	SceneTrackerRefPtr tracker = _tracker.lock();

	if (!tracker || !_IsUsdPrimValid())
	{
		return false;
	}

	// Make sure we've not already advanced to a prepared state.
	if (_phase != _Phase::Started)
	{
		TF_WARN(
			"Invalid call to TrackedPrim.HasUnprocessedPermutations() while "
			"tracking prim <%s> - perhaps PrepareNextPermutation() has already "
			"been invoked?"
			, _prim.GetPrimPath().GetText()
		);
		return false;
	}

	if (tracker->_HasUnprocessedPermutations(_prim))
	{
		_phase = _Phase::Ready;
		return true;
	}

	// no permutations remain, so it's now safe to collect data about
	// this prim (content, overrides, etc)
	//
	// TODO: we don't really need this data until exiting the owning
	// asset or permutation. We could therefore defer this and gather
	// all prims in parallel at that time.
	tracker->_GatherPrimData(_prim);

	_phase = _Phase::Completed;
	return false;
}

bool
TrackedPrim::PrepareNextPermutation()
{
	if (_phase == _Phase::Invalid || _phase == _Phase::Completed)
	{
		return false;
	}

	SceneTrackerRefPtr tracker = _tracker.lock();
	if (!tracker || !_IsUsdPrimValid())
	{
		return false;
	}

	// Check HasUnprocessedPermutations was called previously
	if (_phase < _Phase::Ready)
	{
		TF_WARN(
			"Invalid call to TrackedPrim.PrepareNextPermutation() while tracking "
			"prim <%s> with no permutations to prepare. Please make sure to call "
			"TrackedPrim.HasUnprocessedPermutations() first."
			, _prim.GetPrimPath().GetText()
		);
		return false;
	}

	_phase = _Phase::Preparing;

	// If the permutation state is unset, request the current permutations
	if (!_permState)
	{
		_permState = _PermutationState{ tracker->_GetCurrentPermutations(_prim) };
	}

	// No more permutations so we're finished. False here signals to the caller
	// that there is no additional state for this prim, and it is safe to
	// visit descendants.
	if (_permState->currentIndex >= _permState->perms.size())
	{
		// Note: the phase state gets set to 'Completed' in
		// HasUnprocessedPermutations, not here.
		return false;
	}

	// Otherwise push the next permutation
	tracker->_PushPermutation(_permState->perms.at(_permState->currentIndex++));

	return true;
}

TrackedPrim::operator bool() const noexcept
{
	return static_cast<bool>(_prim);
}

bool
TrackedPrim::_IsUsdPrimValid() const
{
	if (_prim)
	{
		return true;
	}

	TF_WARN(
		"Underlying USD Prim for TrackedPrim <%s> has expired."
		, _prim.GetPath().GetText()
	);
	return false;
}

// static
SceneTrackerRefPtr
SceneTracker::Create(const pxr::UsdStageRefPtr& stage)
{
	return SceneTracker::Create(stage, DiscoveryOptions{});
}

// static
SceneTrackerRefPtr
SceneTracker::Create(const pxr::UsdStageRefPtr& stage,
	                 const DiscoveryOptions& options)
{
	if (!stage)
	{
		TF_RUNTIME_ERROR("Failed to create tracker - USD stage is null.");
		return nullptr;
	}

	const std::string& discoveryPluginName = options.discoveryPluginName;
	DiscoveryPluginRefPtr discoveryPlugin;
	if (discoveryPluginName.empty())
	{
		DEBUG_TRACKING(
		    "Initialized tracker with root layer @%s@ and "
			"default discovery plugin\n"
			, stage->GetRootLayer()->GetIdentifier().c_str()
		);
		discoveryPlugin = DiscoveryPluginRefPtr(new DiscoveryPlugin(options));
	}
	else
	{
		DiscoveryPluginRegistry& registry
			= DiscoveryPluginRegistry::GetInstance();
		discoveryPlugin = registry.Create(options);
		if (discoveryPlugin)
		{
			DEBUG_TRACKING(
				"Initialized tracker with root layer @%s@ and "
				"discovery plugin (%s)\n"
				, stage->GetRootLayer()->GetIdentifier().c_str()
				, discoveryPluginName.c_str()
			);
		}
		else
		{
			TF_RUNTIME_ERROR(
				"Failed to create SceneTracker - failed to find or "
				"create discovery plugin (%s)"
				, discoveryPluginName.c_str()
			);

			return nullptr;
		}
	}
	
	if (!discoveryPlugin->Initialize(stage))
	{
		TF_RUNTIME_ERROR(
			"Failed to create SceneTracker - discovery plugin (%s) "
			"failed to initialize."
			, discoveryPluginName.empty() ? "default" : discoveryPluginName.c_str()
		);
		return nullptr;
	}
	
	return SceneTrackerRefPtr(new SceneTracker(stage, options, discoveryPlugin));
}

SceneTracker::SceneTracker(
	const pxr::UsdStageRefPtr& stage,
	const DiscoveryOptions& options,
	const DiscoveryPluginRefPtr& discoveryPlugin)
	: _stage(stage)
	, _options(options)
	, _discoveryPlugin(discoveryPlugin)
	, _writer(std::make_unique<LayerWriter>(stage))
{
	_SetInitialPath();

	if (_options.purposes.empty())
	{
		_allowedPurposeMask = 0xFF; // all allowed
	}
	else
	{
		// Always include default
		_allowedPurposeMask |= (1 << 0);

		for (const pxr::TfToken& purpose : _options.purposes)
		{
			_allowedPurposeMask |= _ToPurposeMask(purpose);
		}
	}
}

SceneTracker::SceneTracker(const pxr::UsdStageRefPtr& stage)
	: SceneTracker(
		stage,
		DiscoveryOptions{},
		DiscoveryPluginRefPtr(new DiscoveryPlugin({})))
{
}

TrackedPrim
SceneTracker::StartTrackingPrim(const pxr::UsdPrim& prim)
{
	if (!TF_VERIFY(_stage && prim))
	{
		return {};
	}

	const pxr::SdfPath& primPath = prim.GetPrimPath();

	if (ARCH_UNLIKELY(!_errorMark.Check(primPath)))
	{
		return {};
	}

	// Set the first path to the initial tracking request.
	bool isFirstTrackedPrim = false;
	if (_firstTrackedPath.IsEmpty())
	{
		_firstTrackedPath = primPath;
		isFirstTrackedPrim = true;

		if (_firstTrackedPath != _initialPath)
		{
			TF_WARN(
			    "SceneTracker starting at prim <%s> which is different "
				"to the provided initial path <%s>"
				, _firstTrackedPath.GetText()
				, _initialPath.GetText()
			);
		}
	}

	if (_discoveryPlugin->PrunePrim(prim))
	{
		if (ARCH_UNLIKELY(_firstTrackedPath == primPath))
		{
			TF_WARN(
				"The active discovery plugin pruned the initial path <%s>"
				, _initialPath.GetText()
			);
		}

		DEBUG_TRACKING("Pruned prim <%s>", primPath.GetText());

		return {};
	}

	DEBUG_TRACKING(
		"Start tracking %s prim <%s>\n"
		, prim.GetTypeName().GetText()
		, primPath.GetText()
	);

	// If this if the first time visiting the path, collect the
	// default (currently authored) permutation.
	PrimPermutationRefPtr authoredPerm = nullptr;
	if (!_visitedScenePaths.count(primPath))
	{
		DEBUG_TRACKING(
			"Found new scene path <%s>\n"
			, primPath.GetText()
		);

		_visitedScenePaths.insert(primPath);

		PrimPermutation perm = _discoveryPlugin->GetCurrentPrimPermutation(prim);
		if (!perm.IsEmpty())
		{
			authoredPerm = std::make_shared<PrimPermutation>(std::move(perm));
			_permState.emplace(
			    primPath,
			    std::make_unique<_PrimPermutationState>(authoredPerm)
		    );

			DEBUG_PERMUTATION(
				"Got authored prim permutation (uid='%s') for prim <%s>\n"
				, authoredPerm->GetUniqueId().c_str()
				, primPath.GetText()
			);
		}
	}

	// Note: we must ask the discovery plugin for the asset definition on every
	// iteration, since it may change from one permutation to the next. Also,
	// if this is the first prim we need to create a definition regardless of
	// whether or not the discovery plugin broadcast the prim as an asset.
	const bool isAsset = _discoveryPlugin->IsAsset(prim);
	if (isAsset || isFirstTrackedPrim)
	{
		const ExtAssetDefinition* assetDefn
			= isAsset ? _GetDefinitionFromPlugin(prim)
			          : _CreateRootDefinition(prim);

		if (!_PushAsset(prim, assetDefn))
		{
			return {};
		}
	}

	_TrackPurpose(prim);

	if (_options.discoveryMode == DiscoveryMode::AllPermutations)
	{
		_AddPrimPermutations(prim);
	}
	else
	{
		_PushPermutation(authoredPerm);
		// Since we are dealing with a single permutation
		// we can collect prim data now.
		_GatherPrimData(prim);
	}

	return {prim, shared_from_this()};
}

void
SceneTracker::SetTargetCreatedCallback(const TargetCreatedCallback& callback)
{
	_targetCreatedCallback = callback;
}

void
SceneTracker::RemoveTargetCreatedCallback()
{
	_targetCreatedCallback = nullptr;
}

const DiscoveryOptions&
SceneTracker::GetOptions() const
{
	return _options;
}

pxr::SdfPath
SceneTracker::GetInitialPath() const
{
	return _initialPath;
}

TargetDataRefPtr
SceneTracker::GetTargetData(const TargetUid& targetUid) const
{
	return _writer->BuildTargetData(targetUid);
}

bool
SceneTracker::HasErrors() const
{
	return _errorMark.HasErrors();
}

bool
SceneTracker::SaveDataLayer(const std::string& filename) const
{
	return _writer->Save(filename);
}

void
SceneTracker::_SetInitialPath()
{
	_initialPath = [this]() -> pxr::SdfPath
	{
		// Check options
		if (!GetOptions().initialPath.IsEmpty()) {
			return GetOptions().initialPath;
		}

		// Check plugin
		const pxr::SdfPath pluginPath = _discoveryPlugin->GetInitialPath();
		if (!pluginPath.IsEmpty()) {
			return pluginPath;
		}

		// Pseudo root fallback
		return pxr::SdfPath::AbsoluteRootPath();
	}();
}

bool
SceneTracker::_PushAsset(
	const pxr::UsdPrim& prim,
	const ExtAssetDefinition* assetDefn)
{
	const pxr::SdfPath& primPath = prim.GetPrimPath();

	auto _SetErrorMarkAndReturn = [this, &primPath]() -> bool {
		_errorMark.Set(primPath);
		return false;
	};

	if (!TF_VERIFY(prim))
	{
		return _SetErrorMarkAndReturn();
	}

	if (ARCH_UNLIKELY(!assetDefn))
	{
		TF_WARN(
			"Failed to push invalid asset at namespace location <%s>"
			" ... this prim and descendants will not be tracked."
			, prim.GetPrimPath().GetText()
		);

		return _SetErrorMarkAndReturn();
	}

	Item* curItem = _GetCurrentItem();

	if (curItem)
	{
		// Ignore requests to push the same prim. This happens when traversal
		// begins on a fresh permutation of an existing asset.
		if (curItem->GetPath() == prim.GetPrimPath())
		{
			DEBUG_TRACKING(
				"Eliding push asset request for revisited path <%s>\n"
				, prim.GetPrimPath().GetText()
			);

			return true;
		}

		// Otherwise, check the prim we are pushing is a namespace
		// descendant of the current item.
		//
		// TODO: this isn't quite correct, since we should really be checking that
		// the current prim path is an immediate child of the previous path.
		if (!TF_VERIFY(prim.GetPath().HasPrefix(curItem->GetPath()),
			  "Asset prim <%s> is not a namespace descendant of the current "
			  "asset <%s>", prim.GetPath().GetText(), curItem->GetPath().GetText()))
		{
			return _SetErrorMarkAndReturn();
		}
	}

	const pxr::SdfAssetPath& assetIdentifier = assetDefn->GetIdentifier();

	// Issue a warning if we're not the root item, and an identifier fallback is
	// active, and we're unable to find a reference/payload to the identifier.
	if (!_itemStack.empty()
		&& _options.assetIdentifierFallback != IdentifierFallbackMode::None
		&& !util::PrimHasReferenceToAsset(prim, assetIdentifier))
	{
		TF_WARN(
			"Prim <%s> does not reference or payload external "
			"asset (%s) with resolved identifier (%s)"
			, prim.GetPath().GetText()
			, assetDefn->GetUniqueId().c_str()
			, assetIdentifier.GetResolvedPath().c_str()
		);
	}

	_CreateAndPushNewItem(prim, assetDefn);

	if (TfDebug::IsEnabled(PREGEN_TRACKING))
	{
		if (_itemStack.size() == 1)
		{
			DEBUG_TRACKING(
				"Pushed root asset (uid='%s') onto stack\n"
				, _GetCurrentItem()->GetUniqueId().c_str()
			);
		}
		else
		{
			DEBUG_TRACKING(
				"Pushed asset (uid='%s') onto stack - asset hierarchy is now: %s\n"
				, _GetCurrentItem()->GetUniqueId().c_str()
				, _MakeAssetHierarchyString().c_str()
			);
		}
	}

	return true;
}

void
SceneTracker::_AddPrimPermutations(const pxr::UsdPrim& prim)
{
	TF_AXIOM(_options.discoveryMode == DiscoveryMode::AllPermutations);

	const pxr::SdfPath& primPath = prim.GetPrimPath();

	// Firstly, block variant sets we are seeing for the first time. This
	// ensures that nested variant sets that happen to be composed right now
	// will have been removed from the composed prim when we call out to the
	// discovery plugin for its permutations.
	_BlockNewlyDiscoveredVariantSets(prim);

	const std::vector<PrimPermutation> perms
		= _discoveryPlugin->GetPrimPermutations(prim);
	DEBUG_PERMUTATION(
		"Discovered (%zu) permutations for prim <%s>\n"
		, perms.size()
		, primPath.GetText()
	);

	_PrimPermutationState* state = _FindPrimPermutationState(primPath);

	// If there's no permutations and no state and/or permutation set we can
	// exit. (note, if there are no permutations but there is an existing
	// permutation set it must be finalized, which will happen when we pass the
	// empty vector to AddPermutations() in a moment.
	if (perms.empty() && (!state || !state->HasPermutationSet()))
	{
		return;
	}

	// If there's not a state for this prim create one, as well as a permutation
	// set for this path (the constructor that takes an SdfPath creates
	// a permutation set internally)
	if (!state)
	{
		auto [itr, unused] = _permState.emplace(primPath,
			std::make_unique<_PrimPermutationState>(primPath));
		state = itr->second.get();
	}
	// Or, if there's a state but no permutation set create one.
	else if (!state->permSet)
	{
		state->permSet = std::make_unique<PrimPermutationSet>(primPath);
	}

	// Lastly, add the permutations to the set, or finalize the set
	// if passing an empty vector.
	state->permSet->AddPermutations(perms);
}

SceneTracker::_PrimPermutationState*
SceneTracker::_FindPrimPermutationState(const pxr::SdfPath& path)
{
	if (auto itr = _permState.find(path); itr != _permState.end())
	{
		return itr->second.get();
	}

	return nullptr;
}

const SceneTracker::_PrimPermutationState*
SceneTracker::_FindPrimPermutationState(const pxr::SdfPath& path) const
{
	if (const auto itr = _permState.find(path); itr != _permState.cend())
	{
		return itr->second.get();
	}

	return nullptr;
}

bool
SceneTracker::_HasUnprocessedPermutations(const pxr::UsdPrim& prim) const
{
	TF_AXIOM(_options.discoveryMode == DiscoveryMode::AllPermutations);

	if (const _PrimPermutationState* state
		  = _FindPrimPermutationState(prim.GetPath()))
	{
		PrimPermutationSet* permSet = state->permSet.get();
		return permSet ? !permSet->IsEmpty() : false;
	}

	return false;
}

std::vector<PrimPermutationRefPtr>
SceneTracker::_GetCurrentPermutations(const pxr::UsdPrim& prim)
{
	_PrimPermutationState* state = _FindPrimPermutationState(prim.GetPrimPath());
	TF_AXIOM(state && state->HasPermutationSet());
	return state->permSet->GetPermutations();
}

void
SceneTracker::_PushPermutation(const PrimPermutationRefPtr& perm)
{
	if (!perm)
	{
		return;
	}

	DEBUG_TRACKING(
		"Push permutation (uid='%s')\n"
		, perm->GetUniqueId().c_str()
	);

	// Note: the prim here will likely be modified and recomposed once
	// the permutation ops have been applied.
	UsdPrim prim = _stage->GetPrimAtPath(perm->GetPath());
	Item* curItem = _GetCurrentItem();

	TF_AXIOM(prim && curItem);

	ItemScope* curScope = curItem->GetCurrentScope();
	TF_AXIOM(curScope);

	// If we exhausted all ops on the previous iteration we can elide this push.
	if (_options.discoveryMode == DiscoveryMode::AllPermutations)
	{
		const _PermutationApplyResult result = _ApplyPermutationOps(perm);

		if (result == _PermutationApplyResult::Completed)
		{
			return;
		}
		else if (ARCH_UNLIKELY(result == _PermutationApplyResult::Invalid))
		{
			TF_WARN(
				"Failed to apply permutation (uid='%s') at <%s>"
				, perm->GetUniqueId().c_str()
				, perm->GetPath().GetText()
			);
			return;
		}
	}

	const types::int32 nextIndex = _GetNextPermutationIndex();
	const ScopeHandle handle = _writer->AddPermutation(
								   curItem,
								   prim.GetPrimPath(),
								   perm,
								   nextIndex);
	curItem->PushScope(handle,
					   prim.GetPath(),
					   curScope,
					   perm,
					   perm->GetConsumesDescendants());
}

void
SceneTracker::_PopAssetOrPermutation(const pxr::SdfPath& primPath)
{
	TF_AXIOM(primPath.IsAbsoluteRootOrPrimPath());

	Item* curItem = _GetCurrentItem();
	if (!curItem)
	{
		return;
	}

	const pxr::SdfPath& scopePath = curItem->GetCurrentScope()->GetPath();

	// If we're leaving the root scope, clear the permutation set for this prim.
	// This is necessary because we may get reintroduced to a prim at this path
	// in the future, due to new composition arcs on subsequent permutations. If
	// this happens, we don't want the previously visited permutations to be
	// recalled and erroneously skipped.
	//
	// We should also block any authored variant sets here so that revising the
	// same prim (for the same reasons as above) does not appear to have already
	// configured its variants.
	if (_options.discoveryMode == DiscoveryMode::AllPermutations
		&& !curItem->ContainsPermutations())
	{
		if (_PrimPermutationState* state = _FindPrimPermutationState(primPath))
		{
			state->permSet.reset();
		}

		// TODO: it would seem intuitive to clear the set of blocked variant sets
		// here as well - state.blockedVariantSets.clear() - however doing so
		// causes one of the asset switch test cases to fail.
		if (const pxr::UsdPrim prim = _stage->GetPrimAtPath(primPath))
		{
			if (!util::BlockAllVariantSets(prim))
			{
				TF_WARN(
					"Failed to block one or more variant sets on prim <%s>"
					, primPath.GetText()
				);
			}
		}
	}

	// If the prim we're leaving doesn't match the current scope there cannot be
	// a target at this location and/or an item or scope to pop. So, we are safe
	// to early exit. Note however that the permutation state handling above
	// must be called, and so we can't return any earlier than this.
	if (scopePath != primPath)
	{
		return;
	}

	// Generate the target.
	const std::optional<TargetUid> targetUid
		= _writer->GenerateOrReuseTarget(_MakeItemView());

	// Lastly, pop the item or scope stack

	auto _ShouldPopScope = [this, &curItem, &primPath]() -> bool
	{
		// If we are in all-permutations mode, we must keep popping scopes while
		// the item has more permutations to process.
		if (DiscoveryMode::AllPermutations == _options.discoveryMode
			&& curItem->ContainsPermutations())
		{
			return true;
		}

		// Or, in composed-only mode, if the scope path matches the current prim path
		if (DiscoveryMode::ComposedPermutationOnly == _options.discoveryMode
			&& curItem->GetPath() != primPath)
		{
			return true;
		}

		// Otherwise we are done with the item and can remove it from the stack.
		return false;
	};

	if (_ShouldPopScope())
	{
		DEBUG_TRACKING(
			"Popping item-scope (%s) <%s>\n"
			, curItem->GetUniqueId().c_str()
			, curItem->GetCurrentScope()->GetPath().GetText()
		);

		curItem->PopScope();
	}
	else
	{
		DEBUG_TRACKING(
			"Popping item (%s)\n"
			, curItem->GetUniqueId().c_str()
		);

		_identifierInfo.erase(curItem->GetIdentifier().GetResolvedPath());
		_itemStack.pop_back();
	}

	if (targetUid && _targetCreatedCallback)
	{
		_targetCreatedCallback(primPath, *targetUid);
	}
}

void
SceneTracker::_PopPurpose()
{
	if (!_purposeStack.empty())
	{
		_purposeStack.pop_back();
	}
}

types::int32
SceneTracker::_GetOverrideDepthForSpec(const PcpPrimIndex& index,
								  const pxr::SdfPrimSpecHandle& spec) const
{
	PcpNodeRef curNode = index.GetNodeProvidingSpec(spec);

	while (curNode.GetArcType() != PcpArcType::PcpArcTypeRoot)
	{
		const PcpArcType arcType = curNode.GetArcType();
		if (arcType == PcpArcType::PcpArcTypeReference ||
			arcType == PcpArcType::PcpArcTypePayload)
		{
			const std::string resolvedPath
				= pxr::TfNormPath(
				      curNode.GetLayerStack()->GetIdentifier()
				      .rootLayer->GetResolvedPath());

			if (auto itr = _identifierInfo.find(resolvedPath);
				itr != _identifierInfo.end())
			{
				return itr->second.assetDepth;
			}
		}

		curNode = curNode.GetParentNode();
	}

	// Also consult _identifierInfo for the root node's layer. When an asset
	// is opened directly (e.g. worker discovery, no defaultPrim on the asset
	// layer), the asset's own content lives in the root layer with no
	// Reference/Payload arc above it. Without this check the walk above
	// always falls through to the `return 0` fallback and misattributes the
	// asset's intrinsic properties to depth 0 (the entry-point scene root),
	// which then leak into the permutation hash as fake "override" entries
	// and cause the worker to produce a different TargetUid from submission
	// discovery, which reaches the same asset via a reference and never hits
	// this fallback. _identifierInfo entries are now upserted on push (see
	// _PushAsset), so when the synthetic root and the real asset share an
	// identifier (cube.usda for both, etc.), the deepest asset wins and this
	// lookup returns the correct owning depth.
	if (curNode)
	{
		const std::string resolvedPath
			= pxr::TfNormPath(
			      curNode.GetLayerStack()->GetIdentifier()
			      .rootLayer->GetResolvedPath());

		if (auto itr = _identifierInfo.find(resolvedPath);
			itr != _identifierInfo.end())
		{
			return itr->second.assetDepth;
		}
	}

	return 0;
}

void
SceneTracker::_GatherPrimData(const pxr::UsdPrim& prim)
{
	if (!_purposeStack.empty())
	{
		if (const _PurposeInfo& purposeInfo = _purposeStack.back();
			!purposeInfo.isIncluded)
		{
			DEBUG_TRACKING("Skipping prim <%s> due to purpose (%s)\n"
				, prim.GetPrimPath().GetText()
				, purposeInfo.purpose.GetText()
			);
			return;
		}
	}

	DEBUG_TRACKING("Begin gather prim data for <%s>\n", prim.GetPrimPath().GetText());

	Item* curItem = _GetCurrentItem();

	// Note: this function is called directly from TrackedPrim::HasUnprocessedPermutations
	// so there is no guarantee that a valid parent item has been created.
	if (!curItem)
	{
		return;
	}

	ItemScope* curScope = curItem->GetCurrentScope();
	TF_AXIOM(curScope);

	const SdfPath& primPath = prim.GetPrimPath();

	// If we are currently traversing the default variant, tell the writer
	// it can add the scene path to the list of scene nodes.
	if (_ScopeMatchesAuthoredPermutation(curScope))
	{
		_writer->AddNewScenePath(primPath);
	}

	// Ignore overrides on the root asset prim, for now.
	// TODO: should we enforce that this is a transform? The idea is that we
	// can build an actor hierarchy and so things will likely get funky if
	// if we're handed something other than a transform.
	if (curItem->GetPath() == primPath && prim.IsA<pxr::UsdGeomXform>())
	{
		DEBUG_TRACKING(
		    "Ignoring overrides on asset root xform prim <%s>\n"
			, primPath.GetText()
		);
		return;
	}

	DEBUG_TRACKING(
	    "Gathering content and overrides for prim <%s>\n"
		, primPath.GetText()
	);

	// Collect the content category for this prim.
	const TfToken category = _discoveryPlugin->GetContentCategoryForPrim(
							     prim, curItem->GetDefinition());
	if (!category.IsEmpty()
		&& _invalidCategories.find(category) == _invalidCategories.end())
	{
		if (!curScope->AddContent(category, primPath))
		{
			_invalidCategories.insert(category);

			TF_WARN(
		        "Ignoring invalid category identifier '%s' first encountered "
				"while processing prim <%s> of type (%s)"
				, category.GetText()
				, primPath.GetText()
				, prim.GetTypeName().GetText()
		   );
		}
	}

	// Collect the overrides for this prim.
	if (curItem->ShouldCollectOverridesForPrim(primPath))
	{
		const PcpPrimIndex index = prim.GetPrimIndex();
		for (const pxr::SdfPrimSpecHandle& spec : prim.GetPrimStack())
		{
			const types::int32 opinionDepth = _GetOverrideDepthForSpec(index, spec);
			for (const pxr::SdfPropertySpecHandle& prop : spec->GetProperties())
			{
				if(!prop->IsCustom())
				{
					curScope->AddOverridePathIfNeeded(prop->GetPath(), opinionDepth);
				}
			}
		}
	}
}

SceneTracker::_PermutationApplyResult
SceneTracker::_ApplyPermutationOps(const PrimPermutationRefPtr& perm)
{
	if (!TF_VERIFY(perm))
	{
		return _PermutationApplyResult::Invalid;
	}

	Item* curItem = _GetCurrentItem();
	TF_AXIOM(curItem);

	DEBUG_TRACKING("Applying (%zu) permutation ops ...\n", perm->GetOps().size());

	const pxr::SdfPath& primPath = perm->GetPath();

	UsdPrim prim = _stage->GetPrimAtPath(primPath);
	if (!TF_VERIFY(prim,
		"Got invalid/null USD prim from path <%s> held by permutation (uid='%s')"
		, primPath.GetText()
		, perm->GetUniqueId().c_str()))
	{
		return _PermutationApplyResult::Invalid;
	}

	bool reblockVariantSets = false;
	for (const PermutationOpRefPtr& op : perm->GetOps())
	{
		op->Apply(prim);

		// If this permutation contains an inherits op, we need to revisit variantsets,
		// since the inherit arc is stronger, and may have modified the scene description.
		reblockVariantSets |= bool(std::dynamic_pointer_cast<UsdInheritPermutationOp>(op));
	}

	if (reblockVariantSets && !util::BlockAllVariantSets(prim))
	{
		TF_WARN(
			"Failed to block one or more variant sets on prim <%s> while "
			"processing permutations for asset (uid='%s')"
			, primPath.GetText()
			, curItem->GetUniqueId().c_str()
		);
	}

	// Now that we've potentially changed the scene composition by applying the
	// above ops, check the definition again, since the asset may have changed.
	//
	// TODO: we don't really need to check again if there were no composition
	// based ops in the list.
	if (_discoveryPlugin->IsAsset(prim))
	{
		const ExtAssetDefinition* assetDefn = _GetDefinitionFromPlugin(prim);
		if (!assetDefn)
		{
			return _PermutationApplyResult::Invalid;
		}

		if (curItem->GetUniqueId() != assetDefn->GetUniqueId())
		{
			DEBUG_TRACKING(
				"Asset definition changed following permutation op apply - "
				"current definition (uid='%s') vs new definition (uid='%s')\n"
				, curItem->GetUniqueId().c_str()
				, assetDefn->GetUniqueId().c_str()
			);

			// If the switch occurred on a descendant path of the current item
			// we can just let the next iteration of the traversal generate
			// a new item to hold the asset.
			if (curItem->GetPath() != primPath)
			{
				return _PermutationApplyResult::Completed;
			}

			// Otherwise update the current item ...

			_identifierInfo.erase(curItem->GetIdentifier().GetResolvedPath());
			ScopeHandle handle = _writer->AssetDefinitionAdded(*assetDefn);

			const std::optional<pxr::SdfPathSet> overridePaths
				= _discoveryPlugin->GetOverrideCandidatePrimPaths(prim, assetDefn);

			curItem->UpdateAssetDefinition(prim, assetDefn, handle, overridePaths);

			_identifierInfo.insert({curItem->GetIdentifier().GetResolvedPath(),
									{prim.GetPath(), curItem->GetDepth()}});

			_writer->AddChildAssetDependencyToParent(curItem);
		}
	}

	if (!curItem->IsRootItem())
	{
		const PcpNodeRef refNode = util::GetNodeReferencingAsset(prim, curItem->GetIdentifier());
		if (!TF_VERIFY(refNode,
			"Failed to find pcp node with arc-type 'reference', that references "
			"identifier @%s@, as specified by asset (uid='%s')"
			, curItem->GetIdentifier().GetResolvedPath().c_str()
			, curItem->GetUniqueId().c_str()))
		{
			return _PermutationApplyResult::Invalid;
		}

		// Check that any purely variant based permutations actually exist 'inside'
		// the asset. If the variant arc only exists above the asset reference
		// we don't need a permutation generated for it.
		std::set<std::pair<std::string, std::string>> variantsBelowRef;
		auto _VisitNode = [&](auto&& self, const PcpNodeRef& node) -> void
		{
			for (const PcpNodeRef& childNode : node.GetChildrenRange())
			{
				if (childNode.GetArcType() == PcpArcTypeVariant)
				{
					const pxr::SdfPath& variantPath = childNode.GetPath();
					variantsBelowRef.insert(variantPath.GetVariantSelection());
				}
				self(self, childNode);
			}
		};
		_VisitNode(_VisitNode, refNode);

		PermutationOpVector opsToRemove;
		for (const PermutationOpRefPtr& opBase : perm->GetOps())
		{
			if (const auto op = std::dynamic_pointer_cast<UsdVariantPermutationOp>(opBase))
			{
				if (!variantsBelowRef.count(op->GetVariantSelection()))
				{
					opsToRemove.push_back(opBase);
				}
			}
		}

		// TODO: we currently need to remove ops to get the
		// test_asset_switch_on_authored_variant test case to pass. There is
		// probably a better way of tracking the variant hierarchy better
		// instead of modifying the actual permutation.
		perm->RemoveOps(opsToRemove);
		if (perm->IsEmpty())
		{
			return _PermutationApplyResult::Completed;
		}
	}

	return _PermutationApplyResult::InProgress;
}

void
SceneTracker::_BlockNewlyDiscoveredVariantSets(const pxr::UsdPrim& prim)
{
	const pxr::SdfPath& primPath = prim.GetPrimPath();

	_PrimPermutationState* state = _FindPrimPermutationState(primPath);
	if (!state)
	{
		return;
	}

	pxr::UsdVariantSets variantSets = prim.GetVariantSets();
	for (const std::string& variantSetName : variantSets.GetNames())
	{
		if (!state->blockedVariantSets.count(variantSetName))
		{
			DEBUG_TRACKING(
				"Blocking newly discovered variant set (%s)\n"
				, variantSetName.c_str()
			);

			pxr::UsdVariantSet variantSet = variantSets.GetVariantSet(variantSetName);
			if (!variantSet.BlockVariantSelection())
			{
				TF_WARN(
					 "Failed to block variant set (%s) on prim <%s>"
					 , variantSetName.c_str()
					 , primPath.GetText()
				);
			}

			state->blockedVariantSets.insert(variantSetName);
		}
	}
}

Item*
SceneTracker::_GetCurrentItem()
{
	return _itemStack.empty() ? nullptr : _itemStack.back().get();
}

const Item*
SceneTracker::_GetCurrentItem() const
{
	return _itemStack.empty() ? nullptr : _itemStack.back().get();
}

types::int32
SceneTracker::_GetNextPermutationIndex() const
{
	if (const Item* curItem = _GetCurrentItem())
	{
		return static_cast<types::int32>(curItem->NumScopes());
	}

	TF_AXIOM(false);
	return -1;
}

const ExtAssetDefinition*
SceneTracker::_GetDefinitionFromPlugin(const pxr::UsdPrim& prim)
{
	AssetDefinitionRegistry& registry = AssetDefinitionRegistry::GetInstance();
	return registry.AddDefinition(_discoveryPlugin->GetAssetDefinition(prim));
}

const ExtAssetDefinition*
SceneTracker::_CreateRootDefinition(const pxr::UsdPrim& prim)
{
	AssetDefinitionRegistry& registry = AssetDefinitionRegistry::GetInstance();
	return registry.AddDefinition(_discoveryPlugin->CreateRootDefinition(prim));
}

void
SceneTracker::_CreateAndPushNewItem(
	const pxr::UsdPrim& prim,
	const ExtAssetDefinition* assetDefn)
{
	TF_AXIOM(prim && assetDefn);

	ScopeHandle handle = _writer->AssetDefinitionAdded(*assetDefn);

	const std::optional<pxr::SdfPathSet> overridePaths
		= _discoveryPlugin->GetOverrideCandidatePrimPaths(prim, assetDefn);

	Item* parentItem = _GetCurrentItem();
	auto newItem = std::make_unique<Item>(
				       prim, assetDefn, handle, parentItem, overridePaths);

	// insert_or_assign so that when multiple items in the stack share the
	// same identifier (the typical case being a synthetic root and a real
	// asset both keyed to the same asset layer, e.g. when an asset .usd is
	// opened directly with no defaultPrim and discovery synthesises a root
	// definition for the pseudo-root above the real asset prim), the deepest
	// item wins the slot. Override-depth attribution via
	// _GetOverrideDepthForSpec consults this map; leaving the shallower
	// synthetic-root entry in place mis-attributes the asset's intrinsic
	// properties as overrides from depth 0 and pollutes the permutation hash.
	_identifierInfo.insert_or_assign(
		assetDefn->GetIdentifier().GetResolvedPath(),
		_IdentifierEntry{ prim.GetPrimPath(), newItem->GetDepth() }
	);

	_itemStack.push_back(std::move(newItem));

	_writer->AddChildAssetDependencyToParent(_GetCurrentItem());
}

bool
SceneTracker::_ScopeMatchesAuthoredPermutation(const ItemScope* itemScope) const
{
	// TODO: we might need to cache the result here since it gets called for
	// every prim.

	const PrimPermutationConstRefPtr itemPerm = itemScope->GetPermutation();
	if (!itemPerm)
	{
		return true;
	}

	const _PrimPermutationState* state = _FindPrimPermutationState(itemScope->GetPath());
	if (!state || !state->authoredPerm)
	{
		return true;
	}

	return state->authoredPerm->GetUniqueId() == itemPerm->GetUniqueId();
}

void
SceneTracker::_TrackPurpose(const pxr::UsdPrim& prim)
{
	const _PurposeInfo parent =
		_purposeStack.empty()
		? _PurposeInfo{pxr::UsdGeomTokens->default_, false}
		: _purposeStack.back();

	pxr::UsdGeomImageable img(prim);

	pxr::TfToken authoredPurpose;
	if (img)
	{
		pxr::UsdAttribute attr = img.GetPurposeAttr();
		if (attr.HasAuthoredValue())
		{
			attr.Get(&authoredPurpose);
		}
	}

	_PurposeInfo result;

	if (!authoredPurpose.IsEmpty())
	{
		result = { authoredPurpose, true };
	}
	else if (parent.hasAuthoredPurpose)
	{
		result = parent;
	}
	else
	{
		pxr::TfToken fallback = pxr::UsdGeomTokens->default_;
		if (img)
		{
			img.GetPurposeAttr().Get(&fallback);
		}

		result = { fallback, false };
	}

	result.mask = _ToPurposeMask(result.purpose);
	result.isIncluded = (_allowedPurposeMask & result.mask) != 0;

	_purposeStack.push_back(result);
}

std::string
SceneTracker::_MakeAssetHierarchyString() const
{
	std::ostringstream oss;
	for (size_t i = 0; i < _itemStack.size(); ++i)
	{
		oss << _itemStack[i]->GetUniqueId();
		if (i != _itemStack.size() - 1)
		{
			oss << " > ";
		}
	}

	return oss.str();
}

pxr::TfSpan<Item*>
SceneTracker::_MakeItemView() const
{
	_itemStore.clear();
	_itemStore.reserve(_itemStack.size());
	for (const auto& item : _itemStack)
	{
		_itemStore.push_back(item.get());
	}

	return pxr::TfMakeSpan(_itemStore);
}

SceneTracker::_PrimPermutationState::_PrimPermutationState(
	const PrimPermutationConstRefPtr& authoredPerm)
	: authoredPerm(authoredPerm)
{
}

SceneTracker::_PrimPermutationState::_PrimPermutationState(const pxr::SdfPath& path)
	: permSet(std::make_unique<PrimPermutationSet>(path))
{
}

bool
SceneTracker::_PrimPermutationState::HasAuthoredPermutation() const
{
	return authoredPerm != nullptr;
}

bool
SceneTracker::_PrimPermutationState::HasPermutationSet() const
{
	return permSet != nullptr;
}

SceneTracker::_PurposeMask
SceneTracker::_ToPurposeMask(const pxr::TfToken& purpose)
{
	if (purpose == pxr::UsdGeomTokens->default_) return 1 << 0;
	if (purpose == pxr::UsdGeomTokens->render)   return 1 << 1;
	if (purpose == pxr::UsdGeomTokens->proxy)    return 1 << 2;
	if (purpose == pxr::UsdGeomTokens->guide)    return 1 << 3;
	return 0;
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
