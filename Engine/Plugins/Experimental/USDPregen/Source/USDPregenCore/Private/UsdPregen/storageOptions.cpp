// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/storageOptions.h"

#include <sstream>

PREGEN_NAMESPACE_OPEN_SCOPE

bool
StorageOptions::operator==(const StorageOptions& rhs) const
{
	return
		storagePluginName == rhs.storagePluginName &&
		manifestDir == rhs.manifestDir &&
		packageSubPathTemplate == rhs.packageSubPathTemplate;
}

bool
StorageOptions::operator!=(const StorageOptions& rhs) const
{
	return !(*this == rhs);
}

std::string
StorageOptions::DumpToString() const
{
	std::ostringstream os;

	os << "StorageOptions {\n";

	os << "  storagePluginName: \""
	   << storagePluginName << "\"\n";

	os << "  manifestDir: \""
	   << manifestDir << "\"\n";

	os << "  packageSubPathTemplate: \""
	   << packageSubPathTemplate << "\"\n";

	os << "}";

	return os.str();
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
