// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/storagePlugin.h"

#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/manifestTypes.h"
#include "UsdPregen/target.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/token.h"
#include "pxr/base/vt/dictionary.h"
#include "pxr/base/vt/value.h"
#include "USDIncludesEnd.h"

#include <cctype>
#include <mutex>
#include <optional>
#include <sstream>
#include <string>
#include <unordered_set>

PREGEN_NAMESPACE_USING_DIRECTIVE

PXR_NAMESPACE_USING_DIRECTIVE

PREGEN_NAMESPACE_OPEN_SCOPE

namespace
{
	std::string SanitizePackageSegment(const std::string& inValue)
	{
		std::string result;
		result.reserve(inValue.size());

		for (char character : inValue)
		{
			const unsigned char unsignedChar = static_cast<unsigned char>(character);
			if (std::isalnum(unsignedChar) || character == '_')
			{
				result.push_back(character);
			}
			else
			{
				result.push_back('_');
			}
		}

		return result;
	}

	// Convert a leaf VtValue into a string suitable for substitution into a
	// path segment. Non-scalar values (dictionaries, arrays) intentionally
	// produce an empty string so the enclosing `${...}` block collapses,
	// rather than dumping a stringified dict like "{a: 1}" into the path.
	std::string StringifyMetadataValue(const pxr::VtValue& value)
	{
		if (value.IsEmpty())
		{
			return {};
		}
		if (value.IsHolding<pxr::VtDictionary>())
		{
			return {};
		}
		if (value.IsArrayValued())
		{
			return {};
		}
		if (value.IsHolding<std::string>())
		{
			return value.UncheckedGet<std::string>();
		}
		if (value.IsHolding<pxr::TfToken>())
		{
			return value.UncheckedGet<pxr::TfToken>().GetString();
		}

		// Numeric, bool, asset paths, etc.: lean on the registered stream
		// operator. For the types pregen typically sees in assetInfo this
		// yields a sensible representation (e.g. "5", "true", "1.5").
		std::ostringstream os;
		os << value;
		return os.str();
	}

	// Walk a colon-separated path like "category" or "nested:subcategory"
	// through `rootDict`, descending into nested VtDictionary values.
	// Returns the leaf VtValue if the full path resolves, else nullopt.
	std::optional<pxr::VtValue> LookupMetadataPath(
		const pxr::VtDictionary& rootDict,
		const std::string& path)
	{
		if (path.empty())
		{
			return std::nullopt;
		}

		const pxr::VtDictionary* currentDict = &rootDict;
		std::size_t segmentStart = 0;
		while (segmentStart <= path.size())
		{
			std::size_t separator = path.find(':', segmentStart);
			if (separator == std::string::npos)
			{
				separator = path.size();
			}
			const std::string segment = path.substr(segmentStart, separator - segmentStart);
			if (segment.empty())
			{
				return std::nullopt;
			}

			if (!currentDict)
			{
				return std::nullopt;
			}

			const auto entry = currentDict->find(segment);
			if (entry == currentDict->end())
			{
				return std::nullopt;
			}

			const bool isLastSegment = (separator >= path.size());
			if (isLastSegment)
			{
				return entry->second;
			}

			// Not the last segment: must descend into a nested dict.
			if (entry->second.IsHolding<pxr::VtDictionary>())
			{
				currentDict = &entry->second.UncheckedGet<pxr::VtDictionary>();
			}
			else
			{
				return std::nullopt;
			}
			segmentStart = separator + 1;
		}

		return std::nullopt;
	}

	// Look up `key` and return its raw, unsanitized value, or an empty
	// string if the key has no value (or is unknown). ResolveBlock uses
	// the emptiness of the returned value to decide whether to collapse
	// the enclosing `${...}` block.
	std::string ResolveRawPlaceholder(
		const std::string& key,
		const TargetUid& targetUid,
		const ExtAssetDefinition* leafDefinition,
		const std::string& assetType,
		const std::unordered_map<std::string, std::string>& extraSubstitutions)
	{
		const auto extraIt = extraSubstitutions.find(key);
		if (extraIt != extraSubstitutions.end())
		{
			return extraIt->second;
		}

		if (key == "DEFINITION_NAME")
		{
			return leafDefinition ? leafDefinition->GetName() : std::string{};
		}
		if (key == "DEFINITION_VERSION")
		{
			return leafDefinition ? leafDefinition->GetVersion() : std::string{};
		}
		if (key == "DEFINITION_UID")
		{
			return leafDefinition ? leafDefinition->GetUniqueId() : std::string{};
		}
		if (key == "PERMUTATION_ID")
		{
			return targetUid.GetPermutationUid();
		}
		if (key == "ASSET_TYPE")
		{
			return assetType;
		}

		// ${$METADATA:foo} or ${$METADATA:foo:bar} -> walk the leaf
		// definition's metadata dictionary (populated from USD assetInfo,
		// with built-in keys stripped, by the default discovery plugin's
		// _AttachCustomAssetInfoMetadata). Nested dicts are descended via
		// additional ":subkey" segments.
		static const std::string metadataPrefix = "METADATA:";
		if (key.rfind(metadataPrefix, 0) == 0)
		{
			if (!leafDefinition || !leafDefinition->HasMetadata())
			{
				return {};
			}

			const std::string metadataPath = key.substr(metadataPrefix.size());
			const std::optional<pxr::VtValue> resolved = LookupMetadataPath(
				*leafDefinition->GetMetadata(), metadataPath);
			if (!resolved.has_value())
			{
				return {};
			}

			return StringifyMetadataValue(*resolved);
		}

		// Unknown placeholders resolve to empty, which collapses the
		// enclosing `${...}` block in ResolveBlock.
		return {};
	}

	// Emit a one-shot deprecation warning per unique legacy template
	// string. A pregen import resolves the same template hundreds of
	// times (once per produced asset), so without dedup we'd flood the
	// log with the same migration notice on every call.
	void WarnLegacyTemplateOnce(
		const std::string& legacyTemplate,
		const std::string& migratedTemplate)
	{
		static std::mutex warnedMutex;
		static std::unordered_set<std::string> warnedTemplates;

		{
			std::scoped_lock lock(warnedMutex);
			if (!warnedTemplates.insert(legacyTemplate).second)
			{
				return;
			}
		}

		TF_WARN(
			"[USDPregen] Package sub-path template uses the deprecated "
			"`${KEY}` placeholder form. Auto-migrating to the new "
			"`${$KEY}` form for this resolve. Note that support for the old format "
			"may be removed in the future.\n"
			"  before: %s\n"
			"  after:  %s",
			legacyTemplate.c_str(),
			migratedTemplate.c_str());
	}

	// Pre-pass that detects legacy `${KEY}` placeholders inside `templateStr`
	// and rewrites them to the new `${$KEY}` form. Emits a one-shot warning
	// for each unique legacy template seen.
	//
	// Recognized as legacy: a `${...}` block whose content is a non-empty
	// run of identifier characters ([A-Za-z0-9_:]+) with no '$' inside.
	// That catches all the built-in keys (DEFINITION_NAME, PERMUTATION_ID,
	// METADATA:foo:bar, ...) plus identifier-shaped extraSubstitutions.
	// Blocks containing any other character (notably '/') are already in
	// the new form (or simply unrecognized) and are left untouched.
	std::string MigrateLegacyPlaceholders(const std::string& templateStr)
	{
		std::string result;
		result.reserve(templateStr.size());

		bool didMigrate = false;
		std::size_t cursor = 0;
		while (cursor < templateStr.size())
		{
			const char character = templateStr[cursor];
			if (character == '$'
				&& (cursor + 1) < templateStr.size()
				&& templateStr[cursor + 1] == '{')
			{
				const std::size_t closeIndex = templateStr.find('}', cursor + 2);
				if (closeIndex != std::string::npos)
				{
					const std::string blockContent
						= templateStr.substr(cursor + 2, closeIndex - (cursor + 2));

					bool looksLegacy = !blockContent.empty();
					for (char identChar : blockContent)
					{
						const unsigned char unsignedChar
							= static_cast<unsigned char>(identChar);
						if (!std::isalnum(unsignedChar)
							&& identChar != '_'
							&& identChar != ':')
						{
							looksLegacy = false;
							break;
						}
					}

					if (looksLegacy)
					{
						result += "${$";
						result += blockContent;
						result += '}';
						didMigrate = true;
						cursor = closeIndex + 1;
						continue;
					}
				}
			}

			result.push_back(character);
			++cursor;
		}

		if (didMigrate)
		{
			WarnLegacyTemplateOnce(templateStr, result);
		}

		return result;
	}

	// Resolve a `${...}` block.
	//
	// Walks `blockContent`. Each $IDENT (identifier chars + ':' for METADATA
	// paths) is looked up via ResolveRawPlaceholder. If ANY $IDENT resolves
	// to empty, the entire block collapses to an empty string. Otherwise the
	// block is emitted with each $IDENT replaced by its sanitized value;
	// literal characters between $IDENT references pass through verbatim
	// (so the user can author path separators like '/' inside the block).
	//
	// Examples (with DEFINITION_NAME="Ball", DEFINITION_VERSION="2",
	//           PERMUTATION_ID=""):
	//   "$DEFINITION_NAME"                -> "Ball"
	//   "versions/$DEFINITION_VERSION"    -> "versions/2"
	//   "permutations/$PERMUTATION_ID"    -> ""
	std::string ResolveBlock(
		const std::string& blockContent,
		const TargetUid& targetUid,
		const ExtAssetDefinition* leafDefinition,
		const std::string& assetType,
		const std::unordered_map<std::string, std::string>& extraSubstitutions)
	{
		std::string result;
		result.reserve(blockContent.size());
		bool anyEmpty = false;

		std::size_t cursor = 0;
		while (cursor < blockContent.size())
		{
			const char character = blockContent[cursor];
			if (character == '$' && (cursor + 1) < blockContent.size())
			{
				// Identifier characters: letters, digits, underscore, and
				// colon (for METADATA:foo:bar paths)
				std::size_t identStart = cursor + 1;
				std::size_t identEnd = identStart;
				while (identEnd < blockContent.size())
				{
					const char identCharacter = blockContent[identEnd];
					const unsigned char unsignedChar
						= static_cast<unsigned char>(identCharacter);
					if (std::isalnum(unsignedChar)
						|| identCharacter == '_'
						|| identCharacter == ':')
					{
						++identEnd;
					}
					else
					{
						break;
					}
				}

				if (identEnd > identStart)
				{
					const std::string identifier
						= blockContent.substr(identStart, identEnd - identStart);
					const std::string rawValue = ResolveRawPlaceholder(
						identifier,
						targetUid,
						leafDefinition,
						assetType,
						extraSubstitutions);
					if (rawValue.empty())
					{
						anyEmpty = true;
					}
					else
					{
						result += SanitizePackageSegment(rawValue);
					}
					cursor = identEnd;
					continue;
				}
			}

			// Plain character (literal scaffolding inside the block).
			result.push_back(character);
			++cursor;
		}

		if (anyEmpty)
		{
			return {};
		}
		return result;
	}

	// Collapse "//" -> "/" and trim a single trailing slash.
	void CollapseSlashesAndTrim(std::string& path)
	{
		std::string collapsed;
		collapsed.reserve(path.size());
		bool previousWasSlash = false;
		for (char character : path)
		{
			if (character == '/')
			{
				if (previousWasSlash)
				{
					continue;
				}
				previousWasSlash = true;
			}
			else
			{
				previousWasSlash = false;
			}
			collapsed.push_back(character);
		}

		if (!collapsed.empty() && collapsed.back() == '/')
		{
			collapsed.pop_back();
		}

		path = std::move(collapsed);
	}
}	 // namespace

StoragePlugin::StoragePlugin(const StorageOptions& options)
	: _options(options)
{
}

// virtual
StoragePlugin::~StoragePlugin() = default;

// virtual
bool
StoragePlugin::Initialize()
{
	return true;
}

// virtual
bool
StoragePlugin::Shutdown()
{
	return true;
}

// virtual
ManifestSaveResult
StoragePlugin::PersistManifestPayload(const TargetUid& targetUid)
{
	return {};
}

const StorageOptions&
StoragePlugin::GetOptions() const
{
	return _options;
}

// static
const std::string&
StoragePlugin::DefaultPackageSubPathTemplate()
{
	static const std::string sTemplate =
		"assets/${$DEFINITION_NAME}/${versions/$DEFINITION_VERSION}/${permutations/id_$PERMUTATION_ID}";
	return sTemplate;
}

// static
const std::string&
StoragePlugin::EmptyValueSentinel()
{
	static const std::string sSentinel = "_";
	return sSentinel;
}

// static
std::string
StoragePlugin::ResolvePackageSubPathTemplate(
	const std::string& templateStr,
	const TargetUid& targetUid,
	const std::vector<const ExtAssetDefinition*>& definitions,
	const std::string& assetType,
	const std::unordered_map<std::string, std::string>& extraSubstitutions)
{
	const ExtAssetDefinition* leafDefinition
		= definitions.empty() ? nullptr : definitions.back();

	// Pre-pass: rewrite any legacy `${KEY}` placeholders to `${$KEY}` so
	// templates carried over from older saved configs (e.g. a stale
	// USDPregen.ini) keep working. A one-shot deprecation warning is
	// emitted per unique legacy template the first time it's seen.
	//
	// TODO: Remove in a release or two
	const std::string migratedTemplate = MigrateLegacyPlaceholders(templateStr);

	std::string result;
	result.reserve(migratedTemplate.size());

	std::size_t cursor = 0;
	while (cursor < migratedTemplate.size())
	{
		// Look for the opening "${".
		const char character = migratedTemplate[cursor];
		if (character == '$' && (cursor + 1) < migratedTemplate.size() && migratedTemplate[cursor + 1] == '{')
		{
			const std::size_t closeIndex = migratedTemplate.find('}', cursor + 2);
			if (closeIndex != std::string::npos)
			{
				const std::string blockContent
					= migratedTemplate.substr(cursor + 2, closeIndex - (cursor + 2));

				result += ResolveBlock(
					blockContent,
					targetUid,
					leafDefinition,
					assetType,
					extraSubstitutions);
				cursor = closeIndex + 1;
				continue;
			}
		}

		// Plain character (or unmatched "${"): copy verbatim.
		result.push_back(character);
		++cursor;
	}

	CollapseSlashesAndTrim(result);
	return result;
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
