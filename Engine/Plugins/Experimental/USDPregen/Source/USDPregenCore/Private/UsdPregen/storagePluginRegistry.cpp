// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/storagePluginRegistry.h"

#include "UsdPregen/jsonStoragePlugin.h"
#include "UsdPregen/storagePlugin.h"
#include "UsdPregen/storageOptions.h"

#include "USDIncludesStart.h"
#include "pxr/base/arch/defines.h"
#include "pxr/base/arch/env.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/instantiateSingleton.h"
#include "pxr/base/tf/pathUtils.h"
#include "pxr/base/tf/registryManager.h"
#include "pxr/base/tf/singleton.h"
#include "pxr/base/tf/staticTokens.h"
#include "pxr/base/tf/stringUtils.h"
#include "USDIncludesEnd.h"

#include <string>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE // for TF macros

// clang-format off
TF_DEFINE_PRIVATE_TOKENS(
	_tokens,
	(builtin)
	((jsonStorage, "json_storage"))
);
// clang-format on

PREGEN_NAMESPACE_USING_DIRECTIVE

TF_INSTANTIATE_SINGLETON(StoragePluginRegistry);

PREGEN_NAMESPACE_OPEN_SCOPE

namespace
{
	bool _IsReservedPluginName(const std::string& inName)
	{
		return pxr::TfStringStartsWith(inName, _tokens->builtin);
	}
}

StoragePluginRegistry::StoragePluginRegistry()
{
	const bool inserted = _registry.emplace(_tokens->jsonStorage,
		[](const StorageOptions& options) -> StoragePlugin*
		{
			return new JsonStoragePlugin(options);
		}
	).second;

	if (!inserted)
	{
		TF_WARN("Failed to register built-in storage plugin '%s'", _tokens->jsonStorage.GetText());
	}
}

// virtual
bool
StoragePluginRegistry::RegisterFactory(const std::string& pluginName, FactoryFn fn)
{
	if (pluginName.empty() || !fn)
	{
		TF_WARN("Failed to register factory with invalid name and/or callback");
		return false;
	}

	if (_IsReservedPluginName(pluginName))
	{
		TF_WARN(
			"Failed to register storage plugin (%s): '%s' is a reserved prefix."
			, pluginName.c_str()
			, _tokens->builtin.GetText()
		);
		return false;
	}

	DEBUG_REGISTRY(
		"Registering storage plugin (%s)\n"
		, pluginName.c_str()
	);

	auto [itr, inserted] = _registry.emplace(pluginName, fn);
	return inserted;
}

StoragePluginRefPtr
StoragePluginRegistry::Create(const StorageOptions& options) const
{
	pxr::TfRegistryManager::GetInstance().SubscribeTo<StoragePluginRegistry>();

	const std::string& effectivePluginName = options.storagePluginName.empty()
		? _tokens->jsonStorage.GetString()
		: options.storagePluginName;

	auto itr = _registry.find(effectivePluginName);
	if (itr == _registry.end() || itr->second == nullptr)
	{
		TF_WARN(
			"Failed to find a valid factory function for requested "
			"pregen storage plugin (%s)\n"
			, effectivePluginName.c_str()
		);

		return nullptr;
	}

	DEBUG_REGISTRY(
		"Creating storage plugin instance (%s)\n"
		, effectivePluginName.c_str()
	);

	return StoragePluginRefPtr(itr->second(options));
}

// static
StoragePluginRegistry& StoragePluginRegistry::GetInstance()
{
	return TfSingleton<StoragePluginRegistry>::GetInstance();
}

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK