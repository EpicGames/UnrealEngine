// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/target.h"

#include "UsdPregen/assetDefinitionRegistry.h"
#include "UsdPregen/extAssetDefinition.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/sdf/layer.h"
#include "pxr/usd/sdf/path.h"
#include "USDIncludesEnd.h"

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE

PREGEN_NAMESPACE_OPEN_SCOPE

TargetUid::TargetUid(const std::string& definitionUid)
	: _definitionUid(definitionUid)
{
}

TargetUid::TargetUid(const std::string& definitionUid,
					 const std::string& permutationUid)
	: _definitionUid(definitionUid)
	, _permutationUid(permutationUid)
{
}

std::string
TargetUid::GetDefinitionUid() const
{
	return _definitionUid.GetString();
}

std::string
TargetUid::GetPermutationUid() const
{
	return _permutationUid.GetString();
}

bool
TargetUid::HasPermutationUid() const
{
	return !_permutationUid.IsEmpty();
}

std::string
TargetUid::GetString() const
{
	return pxr::TfStringify(*this);
}

bool
TargetUid::IsValid() const
{
	return !_definitionUid.IsEmpty();
}

TargetUid::operator bool() const noexcept
{
	return IsValid();
}

bool
TargetUid::operator==(const TargetUid& rhs) const
{
	return _definitionUid == rhs._definitionUid &&
		   _permutationUid == rhs._permutationUid;
}

bool
TargetUid::operator!=(const TargetUid& rhs) const
{
	return !(*this == rhs);
}

bool
TargetUid::operator<(const TargetUid& rhs) const
{
	if (_definitionUid != rhs._definitionUid)
	{
		return _definitionUid < rhs._definitionUid;
	}

	return _permutationUid < rhs._permutationUid;
}

std::ostream& operator<<(std::ostream& os, const TargetUid& targetUid)
{
	os << targetUid.GetDefinitionUid();

	if (targetUid.HasPermutationUid())
	{
		os << "_" << targetUid.GetPermutationUid();
	}

	return os;
}

// static
TfToken
TargetUid::ConvertToUsdPrimName(const TargetUid& targetUid)
{
	if (!TF_VERIFY(targetUid.IsValid()))
	{
		return {};
	}

	return pxr::TfToken(pxr::TfMakeValidIdentifier(pxr::TfStringify(targetUid)));
}

TargetDefinitionEntry::TargetDefinitionEntry(
	const std::string& defnUid,
	const pxr::SdfPath& scenePath)
	: _defnUid(defnUid)
	, _scenePath(scenePath)
{
}

TargetDefinitionEntry::TargetDefinitionEntry(
	const std::string& defnUid,
	const pxr::SdfPath& scenePath,
	PermutationOpVector permutationOps)
	: _defnUid(defnUid)
	, _scenePath(scenePath)
	, _permutationOps(std::move(permutationOps))
{
}

const ExtAssetDefinition*
TargetDefinitionEntry::GetDefinition() const
{
	AssetDefinitionRegistry& registry = AssetDefinitionRegistry::GetInstance();
	return registry.GetDefinition(_defnUid);
}

const pxr::SdfPath&
TargetDefinitionEntry::GetScenePath() const &
{
	return _scenePath;
}

const PermutationOpVector&
TargetDefinitionEntry::GetPermutationOps() const &
{
	return _permutationOps;
}

TargetData::TargetData(const TargetUid& targetUid)
	: _targetUid(targetUid)
{
}

TargetUid
TargetData::GetUniqueId() const
{
	return _targetUid;
}

std::size_t
TargetData::NumDefinitionEntries() const
{
	return _definitionEntries.size();
}

const TargetDefinitionEntry&
TargetData::GetDefinitionEntry(std::size_t index) const
{
	if (index < _definitionEntries.size())
	{
		return _definitionEntries[index];
	}

	static const TargetDefinitionEntry invalidInfo{};
	return invalidInfo;
}

const std::vector<TargetDefinitionEntry>&
TargetData::GetDefinitionEntries() const &
{
	return _definitionEntries;
}

pxr::SdfLayerRefPtr
TargetData::GetPermutationOverlay() const
{
	return _permOverlayLayer;
}

const std::vector<TargetUid>&
TargetData::GetDependencies() const &
{
	return _dependencies;
}

const pxr::SdfPathSet&
TargetData::GetEncapsulatedDefinitionPaths() const &
{
	return _encapsulatedDefinitions;
}

const pxr::SdfPathSet&
TargetData::GetUnencapsulatedDefinitionPaths() const &
{
	return _unencapsulatedDefinitions;
}

bool
TargetData::IsValid() const
{
	if (!_targetUid.IsValid() || _definitionEntries.empty())
	{
		return false;
	}

	for (const TargetDefinitionEntry& targetInfo : GetDefinitionEntries())
	{
		if (!targetInfo.GetDefinition() || !targetInfo.GetScenePath().IsAbsoluteRootOrPrimPath())
		{
			return false;
		}
	}

	return true;
}

TargetData::operator bool() const noexcept
{
	return IsValid();
}

namespace internal
{

TargetDataBuilder::TargetDataBuilder(const TargetUid& targetUid)
	: _targetData(std::shared_ptr<TargetData>(new TargetData(targetUid)))
{
}

void
TargetDataBuilder::AddInfo(TargetDefinitionEntry info)
{
	_targetData->_definitionEntries.push_back(std::move(info));
}

void
TargetDataBuilder::SetDependencies(std::vector<TargetUid> dependencies)
{
	_targetData->_dependencies = std::move(dependencies);
}

void
TargetDataBuilder::SetEncapsulatedDefinitionPaths(pxr::SdfPathSet paths)
{
	_targetData->_encapsulatedDefinitions = std::move(paths);
}

void
TargetDataBuilder::SetUnencapsulatedDefinitionPaths(pxr::SdfPathSet paths)
{
	_targetData->_unencapsulatedDefinitions = std::move(paths);
}

TargetDataRefPtr
TargetDataBuilder::Build()
{
	return std::move(_targetData);
}

} // namespace internal

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
