// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "util.h"

#include "persistentHasher.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/hash.h"
#include "pxr/base/tf/pathUtils.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/pcp/primIndex.h"
#include "pxr/usd/pcp/layerStack.h"
#include "pxr/usd/pcp/node.h"
#include "pxr/usd/sdf/assetPath.h"
#include "pxr/usd/sdf/attributeSpec.h"
#include "pxr/usd/sdf/layer.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/sdf/primSpec.h"
#include "pxr/usd/sdf/schema.h"
#include "pxr/usd/sdf/valueTypeName.h"
#include "pxr/usd/sdf/variantSpec.h"
#include "pxr/usd/sdf/variantSetSpec.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/primCompositionQuery.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usd/variantSets.h"
#include "USDIncludesEnd.h"

#include <ctime>
#include <chrono>
#include <filesystem>
#include <iomanip>
#include <random>
#include <sstream>
#include <string>
#include <version>

#define PREGEN_ENABLE_DEBUG_MACROS
#include "debugMacros.h"

PXR_NAMESPACE_USING_DIRECTIVE

PREGEN_NAMESPACE_OPEN_SCOPE

namespace internal::util {

pxr::SdfAssetPath
GetAssetFromFirstDefiningSpec(const pxr::UsdPrim& prim)
{
	const pxr::SdfPrimSpecHandleVector primStack = prim.GetPrimStack();

	for (const pxr::SdfPrimSpecHandle& spec : primStack)
	{
		if (spec->GetSpecifier() == pxr::SdfSpecifierDef)
		{
			const pxr::SdfLayerRefPtr layer = spec->GetLayer();
			return {
				layer->GetIdentifier(),
				pxr::TfNormPath(layer->GetResolvedPath().GetPathString())
			};
		}
	}

	return {};
}

/// Returns the first direct reference/payload that successfully resolves
/// to a target layer. Unresolved or invalid arcs are ignored.
pxr::SdfAssetPath
GetAssetFromFirstDirectReferenceOrPayload(const pxr::UsdPrim& prim)
{
	pxr::UsdPrimCompositionQuery::Filter filter;

	filter.arcTypeFilter
	    = pxr::UsdPrimCompositionQuery::ArcTypeFilter::ReferenceOrPayload;

	filter.dependencyTypeFilter
		= pxr::UsdPrimCompositionQuery::DependencyTypeFilter::Direct;

	pxr::UsdPrimCompositionQuery query(prim);
	query.SetFilter(filter);

	for (const pxr::UsdPrimCompositionQueryArc& arc : query.GetCompositionArcs())
	{
		const pxr::SdfLayerRefPtr layer = arc.GetTargetLayer();
		if (!layer)
		{
			continue;
		}

		return {
			layer->GetIdentifier(),
			pxr::TfNormPath(layer->GetResolvedPath().GetPathString())
		};
	}

	return {};
}

pxr::PcpNodeRef
GetNodeReferencingAsset(const pxr::UsdPrim& prim, const pxr::SdfAssetPath& assetPath)
{
	const pxr::PcpPrimIndex index = prim.GetPrimIndex();
	for (pxr::PcpNodeRef node : index.GetNodeRange())
	{
		if (!(node.GetArcType() == pxr::PcpArcType::PcpArcTypeReference
				|| node.GetArcType() == pxr::PcpArcType::PcpArcTypePayload))
		{
			continue;
		}

		const pxr::PcpLayerStackIdentifier layerStack
			= node.GetLayerStack()->GetIdentifier();

		const std::string referenceFile
			= layerStack.rootLayer->GetResolvedPath().GetPathString();

		if (pxr::TfNormPath(referenceFile)
			== pxr::TfNormPath(assetPath.GetResolvedPath()))
		{
			return node;
		}
	}

	return {};
}

void
GetIntroPathAndDefaultPrim(const pxr::UsdPrim& prim,
	                       const pxr::SdfAssetPath& assetPath,
	                       pxr::SdfPath* pathAtIntroduction,
						   TfToken* defaultPrim)
{
	if (!TF_VERIFY(pathAtIntroduction && defaultPrim))
	{
		return;
	}

	const pxr::UsdPrim stageDefaultPrim = prim.GetStage()->GetDefaultPrim();
	if (prim.IsPseudoRoot() || stageDefaultPrim == prim)
	{
		*pathAtIntroduction = prim.GetPrimPath();
		*defaultPrim = prim.GetName();
		return;
	}

	const pxr::PcpNodeRef node = GetNodeReferencingAsset(prim, assetPath);
	if (!node)
	{
		DEBUG_TRACKING(
		    "Prim <%s> does not reference asset path @%s@ - using prim path.\n"
			, prim.GetPrimPath().GetText()
			, assetPath.GetAuthoredPath().c_str()
		);

		*pathAtIntroduction = prim.GetPrimPath();
		*defaultPrim = TfToken();
		return;
	}

	*pathAtIntroduction = node.GetPathAtIntroduction();

	const pxr::PcpLayerStackIdentifier layerStack
		= node.GetLayerStack()->GetIdentifier();
	*defaultPrim = layerStack.rootLayer->GetDefaultPrim();

	DEBUG_TRACKING(
	    "Got path-at-introduction <%s> and default prim (%s) for asset path "
		"@%s@, prim <%s>\n"
		, pathAtIntroduction->GetText()
		, defaultPrim->GetText()
		, assetPath.GetAuthoredPath().c_str()
		, prim.GetPrimPath().GetText()
	);

	return;
}

bool
PrimHasReferenceToAsset(const pxr::UsdPrim& prim, const pxr::SdfAssetPath& assetPath)
{
	return GetNodeReferencingAsset(prim, assetPath) != nullptr;
}

bool
BlockAllVariantSets(const pxr::UsdPrim& prim)
{
	bool blocked = true;
	pxr::UsdVariantSets variantSets = prim.GetVariantSets();
	for (const std::string& variantSetName : variantSets.GetNames())
	{
		pxr::UsdVariantSet variantSet = variantSets.GetVariantSet(variantSetName);
		blocked &= variantSet.BlockVariantSelection();
	}

	return blocked;
}

SdfVariantSetSpecHandle
GetOrCreateVariantSetSpec(
	const SdfSpecHandle& spec,
	const std::string& variantSetName)
{
	if (spec->GetSpecType() == SdfSpecType::SdfSpecTypeVariant)
	{
		SdfVariantSpecHandle variantSpec
			= SdfSpecStatic_cast<SdfVariantSpecHandle>(spec);
		return GetOrCreateVariantSetSpec(variantSpec, variantSetName);
	}
	else if (spec->GetSpecType() == SdfSpecType::SdfSpecTypePrim)
	{
		SdfPrimSpecHandle primSpec = SdfSpecStatic_cast<SdfPrimSpecHandle>(spec);
		return GetOrCreateVariantSetSpec(primSpec, variantSetName);
	}

	return {};
}

SdfVariantSetSpecHandle
GetOrCreateVariantSetSpec(
	const SdfPrimSpecHandle& primSpec,
	const std::string& variantSetName)
{
	const SdfVariantSetsProxy& variantSets = primSpec->GetVariantSets();
	if (auto itr = variantSets.find(variantSetName); itr != variantSets.end())
	{
		return itr->second;
	}

	primSpec->GetVariantSetNameList().Prepend(TfToken(variantSetName));
	return SdfVariantSetSpec::New(primSpec, variantSetName);
}

SdfVariantSetSpecHandle
GetOrCreateVariantSetSpec(
	const SdfVariantSpecHandle& variantSpec,
	const std::string& variantSetName)
{
	const SdfVariantSetsProxy& variantSets = variantSpec->GetVariantSets();
	if (auto itr = variantSets.find(variantSetName); itr != variantSets.end())
	{
		return itr->second;
	}

	variantSpec->GetPrimSpec()
		->GetVariantSetNameList().Prepend(TfToken(variantSetName));
	return SdfVariantSetSpec::New(variantSpec, variantSetName);
}


pxr::SdfVariantSpecHandle
GetOrCreateVariantSpec(
	pxr::SdfVariantSetSpecHandle variantSetSpec,
	const std::string& variantName,
	const std::function<void(const pxr::SdfVariantSpecHandle&)>& onCreate)
{
	auto variants = variantSetSpec->GetVariants();

	if (auto itr = variants.find(variantName); itr != variants.end())
	{
		return *itr;
	}

	pxr::SdfVariantSpecHandle spec =
		pxr::SdfVariantSpec::New(variantSetSpec, variantName);

	if (onCreate)
	{
		onCreate(spec);
	}

	return spec;
}

pxr::SdfVariantSpecHandle
GetOrCreateVariantSpec(
	pxr::SdfVariantSetSpecHandle variantSetSpec,
	const std::string& variantName)
{
	return GetOrCreateVariantSpec(
	   variantSetSpec,
	   variantName,
	   [](const pxr::SdfVariantSpecHandle&) {});
}

pxr::SdfPrimSpecHandle
GetChildSpec(const pxr::SdfPrimSpecHandle& parent,
	const TfToken& childName)
{
	const pxr::SdfChildrenView children = parent->GetNameChildren();

	if (auto childSpec = children.find(childName); childSpec != children.end())
	{
		return *childSpec;
	}

	return pxr::SdfPrimSpecHandle();
}

std::pair<std::string, std::uint64_t>
GetLocalLayerStackTimestampAndHash(
	const pxr::UsdStageConstRefPtr& stage,
	bool useRealPaths)
{
	using std::chrono::system_clock;
	using std::filesystem::file_time_type;

	PersistentHasher hasher;

	if (!stage)
	{
		return {};
	}

	const pxr::SdfLayerHandleVector layerStack
		= stage->GetLayerStack(/*includeSessionLayers=*/false);

	std::uint64_t maxTime = 0;

	for (const auto& layer : layerStack)
	{
		const std::string realPath = layer->GetRealPath();
		if (realPath.empty())
		{
			continue;
		}

		const std::string pathToHash = useRealPaths
			? realPath
			: layer->GetIdentifier();

		DEBUG_ASSET("Hashing sublayer path @%s@\n", pathToHash.c_str());
		hasher.AddFilePath(pathToHash);

		std::error_code error;
		const file_time_type fmodTime
			= std::filesystem::last_write_time(realPath, error);
		if (error)
		{
			TF_WARN(
				"Failed to get last write time for file (%s) - %s"
				, realPath.c_str()
				, error.message().c_str()
			);
			continue;
		}

		const std::uint64_t modTimeSeconds
			= FileTimeToSystemTimeSeconds(fmodTime);
		hasher.AddUint64(modTimeSeconds);

		maxTime = std::max(maxTime, modTimeSeconds);
	}

	const std::string latestTimestamp = FormatTimestamp(maxTime);
	const std::uint64_t hash = hasher.GetHash64();

	return std::make_pair(latestTimestamp, hash);
}

std::uint64_t
FileTimeToSystemTimeSeconds(const std::filesystem::file_time_type& fileTime)
{
	using namespace std::chrono;

#if defined(__cpp_lib_chrono) && (__cpp_lib_chrono >= 201907L)
	// C++20 drift-free conversion
	const auto sysTime = floor<seconds>(clock_cast<system_clock>(fileTime));
#else
	// In C++17, there is no std::chrono::file_clock
	// We use the clock type defined by the filesystem instead.
	using FileClock = std::filesystem::file_time_type::clock;

	// C++17 manual offset calculation
	// Note: Use FileClock::now() because std::chrono::file_clock doesn't exist
	const auto nowFile = FileClock::now();
	const auto nowSys  = system_clock::now();

	// Floor directly to seconds. Capturing the floored result with `auto`
	// keeps sysTime typed as time_point<system_clock, seconds>, so .count()
	// below returns a count in seconds. Assigning the floored value back into
	// a wider-duration variable would convert it back (e.g. multiplying by
	// 1e6 if system_clock::duration is microseconds), and the returned value
	// would no longer be in seconds.
	const auto sysTime = floor<seconds>(fileTime - nowFile + nowSys);
#endif

	return static_cast<std::uint64_t>(sysTime.time_since_epoch().count());
}

std::string
FormatTimestamp(const std::uint64_t seconds)
{
	using std::chrono::system_clock;
	using std::filesystem::file_time_type;

	if (seconds == 0)
	{
		return "0000-00-00_00-00-00";
	}

	const std::time_t epochTime = static_cast<std::time_t>(seconds);
	std::tm timeInfo{};
	bool success = false;

#if defined(_WIN32)
	// Windows thread-safe UTC time
	success = (gmtime_s(&timeInfo, &epochTime) == 0);
#else
	// POSIX thread-safe UTC time
	success = (gmtime_r(&epochTime, &timeInfo) != nullptr);
#endif

	if (!success) {
		return "error_invalid_time";
	}

	std::ostringstream oss;
	// Format: YYYY-MM-DD_HH-MM-SS (Safe for Windows drive/folder naming)
	oss << std::put_time(&timeInfo, "%Y-%m-%d_%H-%M-%S");
	return oss.str();
}

uint64_t
RandomUInt64()
{
	thread_local std::mt19937_64 rng(std::random_device{}());
	static std::uniform_int_distribution<uint64_t> dist;
	return dist(rng);
}

uint32_t
HashStringArrayToUInt32(const std::vector<std::string>& values)
{
	const std::size_t h = pxr::TfHash()(values);

	std::uint64_t x = static_cast<std::uint64_t>(h);

	x ^= x >> 33;
	x *= 0xff51afd7ed558ccdULL;
	x ^= x >> 33;
	x *= 0xc4ceb9fe1a85ec53ULL;
	x ^= x >> 33;
	return static_cast<std::uint32_t>(x);
}


// ===================================================================== //
//                  metadataBlob round-trip helpers                       //
// ===================================================================== //

std::string
MetadataToUsdaBlob(const pxr::VtDictionary& metadata)
{
	// Empty dict round-trips as an empty string - no point in serializing an
	// otherwise-empty USDA layer.
	if (metadata.empty())
	{
		return {};
	}

	pxr::SdfLayerRefPtr layer = pxr::SdfLayer::CreateAnonymous(".usda");
	if (!layer)
	{
		TF_WARN("MetadataToUsdaBlob: failed to create scratch layer");
		return {};
	}

	layer->SetCustomLayerData(metadata);

	std::string out;
	if (!layer->ExportToString(&out))
	{
		TF_WARN("MetadataToUsdaBlob: failed to export layer to string");
		return {};
	}

	return out;
}

std::optional<pxr::VtDictionary>
UsdaBlobToMetadata(const std::string& blob)
{
	// Empty string is the canonical encoding for an empty VtDictionary
	// (see MetadataToUsdaBlob).
	if (blob.empty())
	{
		return pxr::VtDictionary{};
	}

	pxr::SdfLayerRefPtr layer = pxr::SdfLayer::CreateAnonymous(".usda");
	if (!layer)
	{
		TF_WARN("UsdaBlobToMetadata: failed to create scratch layer");
		return std::nullopt;
	}

	if (!layer->ImportFromString(blob))
	{
		TF_WARN("UsdaBlobToMetadata: failed to import metadata blob");
		return std::nullopt;
	}

	return layer->GetCustomLayerData();
}


// ===================================================================== //
//             SerializedOp / PermutationOp (de)serialization             //
// ===================================================================== //

namespace
{

constexpr const char* _OpargsPrefix = "opargs:";
constexpr std::size_t _OpargsPrefixLen = 7;

constexpr const char* _TempPrimName = "__pregen_op_spec__";

// Constructs an anonymous layer with a single root prim spec used as a
// temporary scratch space for op serialization.
pxr::SdfPrimSpecHandle
_CreateTempOpPrimSpec(pxr::SdfLayerRefPtr& outLayer)
{
	outLayer = pxr::SdfLayer::CreateAnonymous();
	if (!outLayer)
	{
		return {};
	}

	const pxr::SdfPath primPath = pxr::SdfPath::AbsoluteRootPath().AppendChild(
		pxr::TfToken(_TempPrimName));
	pxr::SdfPrimSpecHandle primSpec
		= pxr::SdfCreatePrimInLayer(outLayer, primPath);
	return primSpec;
}

} // anonymous namespace

SerializedOp
ToSerializedOp(const PermutationOpRefPtr& op)
{
	SerializedOp result;

	if (!op)
	{
		return result;
	}

	pxr::SdfLayerRefPtr scratchLayer;
	pxr::SdfPrimSpecHandle primSpec = _CreateTempOpPrimSpec(scratchLayer);
	if (!primSpec)
	{
		TF_WARN("ToSerializedOp: failed to create scratch prim spec");
		return result;
	}

	op->Serialize(primSpec);

	result.typeName = primSpec->GetTypeName();

	for (const pxr::SdfPropertySpecHandle& prop : primSpec->GetProperties())
	{
		if (!prop)
		{
			continue;
		}

		const std::string& propName = prop->GetName();
		if (propName.size() <= _OpargsPrefixLen
			|| propName.compare(0, _OpargsPrefixLen, _OpargsPrefix) != 0)
		{
			continue;
		}

		const pxr::SdfAttributeSpecHandle attr
			= pxr::TfDynamic_cast<pxr::SdfAttributeSpecHandle>(prop);
		if (!attr)
		{
			continue;
		}

		SerializedOpArg arg;
		arg.name = propName.substr(_OpargsPrefixLen);
		arg.typeName = attr->GetTypeName().GetAsToken().GetString();

		const pxr::VtValue defaultValue = attr->GetDefaultValue();
		if (defaultValue.IsHolding<std::string>())
		{
			arg.value = defaultValue.UncheckedGet<std::string>();
		}
		else
		{
			// Best-effort stringification for non-string scalar types. Custom
			// op authors should prefer string-typed opargs for portability;
			// non-string types may not round-trip exactly.
			arg.value = pxr::TfStringify(defaultValue);
		}

		result.args.push_back(std::move(arg));
	}

	return result;
}

PermutationOpRefPtr
FromSerializedOp(const SerializedOp& serializedOp)
{
	if (serializedOp.typeName.empty())
	{
		return nullptr;
	}

	pxr::SdfLayerRefPtr scratchLayer;
	pxr::SdfPrimSpecHandle primSpec = _CreateTempOpPrimSpec(scratchLayer);
	if (!primSpec)
	{
		TF_WARN("FromSerializedOp: failed to create scratch prim spec");
		return nullptr;
	}

	primSpec->SetTypeName(serializedOp.typeName);

	const pxr::SdfSchema& schema = pxr::SdfSchema::GetInstance();

	for (const SerializedOpArg& arg : serializedOp.args)
	{
		const pxr::SdfValueTypeName valueType
			= schema.FindType(pxr::TfToken(arg.typeName));
		if (!valueType)
		{
			TF_WARN(
				"FromSerializedOp: unrecognized SdfValueTypeName '%s' for "
				"arg '%s' on op type '%s' - skipping",
				arg.typeName.c_str(),
				arg.name.c_str(),
				serializedOp.typeName.c_str()
			);
			continue;
		}

		const std::string fullName = std::string(_OpargsPrefix) + arg.name;
		pxr::SdfAttributeSpecHandle attr
			= pxr::SdfAttributeSpec::New(primSpec, fullName, valueType);
		if (!attr)
		{
			continue;
		}

		// All current op authors use string-typed opargs. For non-string
		// types we fall back to leaving the default unset; the factory will
		// surface the failure if the value is required.
		if (arg.typeName == "string")
		{
			attr->SetDefaultValue(pxr::VtValue(arg.value));
		}
		else
		{
			TF_WARN(
				"FromSerializedOp: non-string opargs type '%s' on arg '%s' "
				"is not supported in v1 - skipping value",
				arg.typeName.c_str(),
				arg.name.c_str()
			);
		}
	}

	return PermutationOp::CreateFromSpec(primSpec);
}

} // namespace internal::util

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
