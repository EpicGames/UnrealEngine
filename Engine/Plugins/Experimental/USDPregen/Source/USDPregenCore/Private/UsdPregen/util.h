// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"
#include "UsdPregen/permutationOps.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/vt/dictionary.h"
#include "USDIncludesEnd.h"

#include <cstdint>
#include <filesystem>
#include <functional>
#include <optional>
#include <string>
#include <utility>
#include <vector>

#define PREGEN_API USDPREGENCORE_API

PXR_NAMESPACE_OPEN_SCOPE

	class PcpNodeRef;
	class SdfAssetPath;
	class SdfPath;
	class SdfSpec;
	class SdfPrimSpec;
	class SdfVariantSpec;
	class SdfVariantSetSpec;
	class TfToken;
	class UsdPrim;
	class UsdStage;

	template <typename T>
	class SdfHandle;

	using SdfSpecHandle = SdfHandle<SdfSpec>;
	using SdfPrimSpecHandle = SdfHandle<SdfPrimSpec>;
	using SdfVariantSpecHandle = SdfHandle<SdfVariantSpec>;
	using SdfVariantSetSpecHandle = SdfHandle<SdfVariantSetSpec>;

	template <typename T>
	class TfRefPtr;

	using UsdStageConstRefPtr = TfRefPtr<const UsdStage>;

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

namespace internal::util
{

pxr::SdfAssetPath
GetAssetFromFirstDefiningSpec(const pxr::UsdPrim& prim);

pxr::SdfAssetPath
GetAssetFromFirstDirectReferenceOrPayload(const pxr::UsdPrim& prim);

pxr::PcpNodeRef
GetNodeReferencingAsset(
	const pxr::UsdPrim& prim,
	const pxr::SdfAssetPath& assetPath);

void
GetIntroPathAndDefaultPrim(
	const pxr::UsdPrim& prim,
	const pxr::SdfAssetPath& assetPath,
	pxr::SdfPath* targetedPrimPath,
	pxr::TfToken* defaultPrim);

bool
PrimHasReferenceToAsset(
	const pxr::UsdPrim& prim,
	const pxr::SdfAssetPath& assetPath);

template <typename T>
std::vector<std::vector<T>>
CartesianProduct(const std::vector<std::vector<T>>& sets)
{
	std::vector<std::vector<T>> result;
	std::vector<T> current;

	std::function<void(size_t)> recurse = [&](size_t depth)
	{
		if (depth == sets.size()) {
			result.push_back(current);
			return;
		}

		for (const auto& value : sets[depth]) {
			current.push_back(value);
			recurse(depth + 1);
			current.pop_back();
		}
	};

	recurse(0);
	return result;
}

bool
BlockAllVariantSets(const pxr::UsdPrim& prim);

pxr::SdfVariantSetSpecHandle
GetOrCreateVariantSetSpec(
	const pxr::SdfSpecHandle& spec,
	const std::string& variantSetName);

pxr::SdfVariantSetSpecHandle
GetOrCreateVariantSetSpec(
	const pxr::SdfPrimSpecHandle& primSpec,
	const std::string& variantSetName);

pxr::SdfVariantSetSpecHandle
GetOrCreateVariantSetSpec(
	const pxr::SdfVariantSpecHandle& variantSpec,
	const std::string& variantSetName);

pxr::SdfVariantSpecHandle
GetOrCreateVariantSpec(
	pxr::SdfVariantSetSpecHandle variantSetSpec,
	const std::string& variantName,
	const std::function<void(const pxr::SdfVariantSpecHandle&)>& onCreate);

pxr::SdfVariantSpecHandle
GetOrCreateVariantSpec(
	pxr::SdfVariantSetSpecHandle variantSetSpec,
	const std::string& variantName);

pxr::SdfPrimSpecHandle
GetChildSpec(
	const pxr::SdfPrimSpecHandle& parent,
	const pxr::TfToken& childName);

std::pair<std::string, std::uint64_t>
GetLocalLayerStackTimestampAndHash(
	const pxr::UsdStageConstRefPtr& stage,
	bool hashRealPaths = false);

std::uint64_t
FileTimeToSystemTimeSeconds(const std::filesystem::file_time_type& fileTime);

std::string
FormatTimestamp(const std::uint64_t seconds);

std::uint64_t
RandomUInt64();

std::uint32_t
HashStringArrayToUInt32(const std::vector<std::string>& values);


/// Round-trips an arbitrary VtDictionary through an anonymous USDA-format
/// SdfLayer's customLayerData field.
///
/// The resulting string is a USDA snippet that consumers can persist as a
/// single opaque blob. Avoids requiring a custom JSON / UStruct converter
/// for the many USD value types VtDictionary can hold.
///
/// An empty input dictionary serializes to an empty string (rather than a
/// USDA layer with no customLayerData), so storage backends can persist
/// "no metadata" as the empty string in their wire format.
///
/// Returns an empty string on failure; a warning is logged via TF_WARN.
PREGEN_API std::string
MetadataToUsdaBlob(const pxr::VtDictionary& metadata);

/// Inverse of MetadataToUsdaBlob.
///
/// An empty input string decodes to an empty VtDictionary (paired with
/// MetadataToUsdaBlob's empty-dict-as-empty-string convention).
///
/// Returns std::nullopt only on parse failure of a non-empty blob (a
/// warning is logged in that case).
PREGEN_API std::optional<pxr::VtDictionary>
UsdaBlobToMetadata(const std::string& blob);


/// \struct SerializedOpArg
///
/// Wire-format representation of a single argument extracted from the
/// "opargs:" namespace of a serialized PermutationOp prim spec.
struct SerializedOpArg
{
	/// Argument name with the "opargs:" prefix removed.
	std::string name;

	/// Sdf value type name (see SdfValueTypeNames). For example "string" or
	/// "token". The deserializer uses this to construct the matching
	/// SdfAttributeSpec.
	std::string typeName;

	/// String-encoded default value. Round-tripped via VtValue stringification.
	std::string value;
};

/// \struct SerializedOp
///
/// Wire-format representation of a single PermutationOp.
///
/// Captures the prim-spec typeName authored by the op's Serialize() override
/// and the list of opargs:* arguments. Intended as a shared intermediate
/// between the JSON and UObject manifest backends so they can share the
/// op (de)serialization logic.
struct SerializedOp
{
	/// Prim-spec type name authored by the op's Serialize() override.
	/// For example "UsdVariantSelectionOp" or "UsdInheritOp".
	std::string typeName;

	/// Argument list extracted from the prim spec's "opargs:" namespace.
	std::vector<SerializedOpArg> args;
};

/// Serializes a PermutationOp into the shared wire-format representation.
///
/// Internally creates an anonymous SdfLayer, asks the op to serialize itself
/// into a temporary prim spec, and then extracts the spec's type name and
/// opargs:* attributes.
///
/// Returns an empty SerializedOp (typeName empty) on failure.
PREGEN_API SerializedOp
ToSerializedOp(const PermutationOpRefPtr& op);

/// Reconstructs a PermutationOp from the shared wire-format representation.
///
/// Creates an anonymous SdfPrimSpec with the supplied typeName and opargs:*
/// attributes, then dispatches to PermutationOp::CreateFromSpec which looks
/// up the type-name-keyed factory registered by the op subclass.
///
/// Returns nullptr if no factory is registered for the type name.
PREGEN_API PermutationOpRefPtr
FromSerializedOp(const SerializedOp& serializedOp);

} // namespace internal::util

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
