// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/weakBase.h"
#include "USDIncludesEnd.h"

#include <memory>
#include <mutex>
#include <string>
#include <unordered_map>
#include <vector>

#define PREGEN_API USDPREGENCORE_API

PXR_NAMESPACE_OPEN_SCOPE

class TfToken;
class TfHash;

template <typename T>
class TfSingleton;

template <typename T>
class TfRefPtr;

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

class ExtAssetDefinition;

/// \class AssetDefinitionRegistry
///
/// Registry of ExtAssetDefinition objects.
///
/// AssetDefinitionRegistry stores and provides lookup access to
/// asset definitions identified by a unique identifier string.
/// Definitions are typically registered during initialization or
/// discovery phases and remain owned by the registry for the
/// lifetime of the process.
///
/// The registry is implemented as a TfSingleton and can be accessed
/// via GetInstance().
class AssetDefinitionRegistry : public pxr::TfWeakBase
{
public:

	/// Returns the global AssetDefinitionRegistry instance.
	PREGEN_API static AssetDefinitionRegistry& GetInstance();

	/// Registers an asset definition with the registry.
	///
	/// If a definition with the same identifier already exists, the
	/// registry will compare the definitions fields, including custom
	/// metadata, and return the existing definition if both are
	/// identical. If mismatching fields are encountered nullptr will
	/// be returned.
	///
	/// \param assetDefn Asset definition to register.
	///
	/// \return Pointer to the stored definition instance, or nullptr
	/// if the definition could not be registered.
	PREGEN_API const ExtAssetDefinition* AddDefinition(
						 const ExtAssetDefinition& assetDefn);

	/// Returns the asset definition associated with the given identifier.
	///
	/// \param uniqueId Unique identifier of the asset definition.
	///
	/// \return Pointer to the registered definition, or nullptr if
	/// no matching definition exists.
	PREGEN_API const ExtAssetDefinition* GetDefinition(
						 const std::string& uniqueId) const;

	/// Returns all the asset definitions present in the registry
	///
	/// \return Vector of definition pointers.
	PREGEN_API std::vector<const ExtAssetDefinition*>
	GetAllDefinitions() const;

private:

	friend class pxr::TfSingleton<AssetDefinitionRegistry>;

	friend struct _DefinitionRegistryTestHelper;

	AssetDefinitionRegistry() = default;
	~AssetDefinitionRegistry() = default;

	/// Registry storage mapping identifiers to definitions.
	std::unordered_map<pxr::TfToken,
					   std::unique_ptr<ExtAssetDefinition>,
					   pxr::TfHash> _entries;

	mutable std::mutex _mutex;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
