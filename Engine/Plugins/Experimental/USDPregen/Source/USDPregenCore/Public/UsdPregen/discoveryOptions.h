// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/usd/primFlags.h"
#include "USDIncludesEnd.h"

#include <cstdint>
#include <string>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

/// Controls how permutations are explored during discovery.
enum class DiscoveryMode : std::uint8_t
{
	/// Explore all permutations discovered for a prim.
	AllPermutations,

	/// Only process the currently composed configuration.
	ComposedPermutationOnly
};

/// Controls how version strings are generated in the
/// absence of explicit metadata.
enum class VersionFallbackMode : std::uint8_t
{
	/// No fallback
	None,

	/// Build a version hash using the asset's local layer stack
	/// identifiers and timestamps.
	LayerStackFilesAndTimestamps,

	/// Build a version hash using the asset's local layer stack
	/// resolved paths and timestamps.
	ResolvedLayerStackFilesAndTimestamps
};

/// Controls how asset identifiers are generated in the absence
/// of an explicitly provided asset path.
enum class IdentifierFallbackMode : std::uint8_t
{
	/// No fallback - an explicit, resolved SdfAssetPath must be provided.
	None,

	/// Inspect the prim composition and extract the first direct reference
	/// or payload, if any.
	FirstDirectReferenceOrPayload
};

/// Configuration options controlling pregen tracker and discovery behavior.
struct DiscoveryOptions
{
	/// Permutation discovery mode.
	DiscoveryMode discoveryMode = DiscoveryMode::AllPermutations;

	/// Name of the discovery plugin to use. An empty string will use
	/// the built-in discovery plugin.
	std::string discoveryPluginName;

	/// A prefix that may be added to all newly created asset definition names.
	/// This allows for per-project definition UIDs, for example, or creating
	/// test assets using a temporary prefix. The default discovery plugin will
	/// always prepend the prefix, however custom plugins must manually validate
	/// and incorporate the prefix into the definitions they generate.
	std::string definitionPrefix;

	/// The path to use when starting the traversal. This allows discovery to
	/// be limited to a subsection of the input stage. Note that this value
	/// overrides the initial path specified by the discovery plugin.
	pxr::SdfPath initialPath;

	/// The UsdGeomImageable purposes to include during discovery. An empty
	/// vector is interpreted as all purposes.
	pxr::TfTokenVector purposes;

	/// A list of variant set names that should be ignored during discovery.
	/// This is typically used to filter out variant sets that are not relevant
	/// for asset permutations. An entry of "*" can be used to exclude all
	/// variant sets.
	pxr::TfTokenVector excludeVariantSets;

	/// When an asset does not specify an explicit identifier, controls
	/// the strategy used to compute a fallback value.
	IdentifierFallbackMode assetIdentifierFallback = IdentifierFallbackMode::None;

	/// When an asset does not specify a version, controls the strategy used
	/// to compute a fallback.
	VersionFallbackMode assetVersionFallback = VersionFallbackMode::None;

	/// The predicate to use during traversal. This allows abstract or unloaded
	/// prims to be discovered if needed.
	pxr::Usd_PrimFlagsPredicate traversalPredicate = pxr::UsdPrimDefaultPredicate;

	PREGEN_API bool operator==(const DiscoveryOptions& rhs) const;

	PREGEN_API bool operator!=(const DiscoveryOptions& rhs) const;

	PREGEN_API std::string DumpToString() const;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
