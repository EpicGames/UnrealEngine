// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "UsdPregen/discoveryOptions.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/sdf/assetPath.h"
#include "USDIncludesEnd.h"

#include <memory>
#include <optional>
#include <vector>

#define PREGEN_API USDPREGENCORE_API

PXR_NAMESPACE_OPEN_SCOPE

	template <typename T>
	class TfRefPtr;

	template <typename T>
	class SdfHandle;

	class SdfPath;
	class SdfPrimSpec;
	class TfToken;
	class UsdPrim;
	class UsdStage;
	class VtDictionary;
	class VtValue;

	using UsdStageRefPtr = TfRefPtr<UsdStage>;
	using SdfPrimSpecHandle = SdfHandle<SdfPrimSpec>;

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

namespace DiscoveryPluginTokens
{
	extern PREGEN_API const pxr::TfToken reservedMetadataPrefix;
	extern PREGEN_API const pxr::TfToken definitionPrefix;
	extern PREGEN_API const pxr::TfToken initialPrim;
}

namespace DefaultCategoryTokens
{
	extern PREGEN_API const pxr::TfToken boundables;
	extern PREGEN_API const pxr::TfToken materials;
	extern PREGEN_API const pxr::TfToken meshes;
	extern PREGEN_API const pxr::TfToken shaders;
	extern PREGEN_API const pxr::TfToken skeletons;
	extern PREGEN_API const pxr::TfToken skelAnimations;
	extern PREGEN_API const pxr::TfToken skelRoots;
	extern PREGEN_API const pxr::TfToken lights;
	extern PREGEN_API const pxr::TfToken noCategory;
}

namespace internal { struct _VersionCache; }

class ExtAssetDefinition;
class PrimPermutation;

using DiscoveryPluginRefPtr = std::shared_ptr<class DiscoveryPlugin>;

/// \class DiscoveryPlugin
///
/// Base class for asset discovery plugins.
///
/// A DiscoveryPlugin identifies asset entry points in a USD scene. During
/// scene traversal the plugin is queried to determine whether a prim represents
/// an asset, and if so to produce an ExtAssetDefinition. A plugin may also
/// broadcast multiple permutations of an asset to the system.
///
/// Only a single discovery plugin is active for a given discovery session
/// (typically a composed UsdStage). The active plugin determines the
/// interpretation of assets and permutations for the entire traversal.
///
/// Plugins may be implemented in either C++ or Python and are typically
/// instantiated through the DiscoveryPluginRegistry.
///
/// The default implementation provides a functional discovery policy:
///
/// Asset detection uses UsdModelAPI `assetInfo` metadata (`name`, `version`
/// and `identifier`) when available. The only mandatory field is `name`. If
/// `version` is unauthored it can remain unauthored (empty string), or a
/// version can be constructed by setting the version fallback mode in the
/// discovery options.
///
/// If asset identifier is unauthored, the definition will be invalid unless
/// an identifier can be constructed. The identifier fallback mode in the
/// discovery options can be used to deduce an identifier.
///
/// Permutations are automatically derived from variant sets on the prim. The
///	currently selected variants define the current permutation, while unselected
///	variants may generate additional permutations depending on the current
/// discovery mode.
///
/// Custom implementations may override only the portions of the behavior they
/// wish to specialize. For example, a plugin may override IsAsset() and
/// GetAssetDefinition() in order to utilize custom prim metadata, while relying
/// on the default permutation handling.
///
/// Custom implementations can attach metadata to the generated definitions.
/// The metadata will be available during the storage phase and can be used to
/// organize product storage locations by asset category, for example.
class DiscoveryPlugin
{
public:

	PREGEN_API DiscoveryPlugin(const DiscoveryOptions& options);

	PREGEN_API virtual ~DiscoveryPlugin();

	/// Initialize any internal data, prior to traversal.
	PREGEN_API virtual bool Initialize(const pxr::UsdStageRefPtr& stage);

	/// Get the initial path from which to start traversals.
	PREGEN_API virtual pxr::SdfPath GetInitialPath() const;

	/// Returns true if the given prim represents an asset.
	///
	/// The default implementation considers a prim to be an asset when the
	/// prim's asset info metadata has an authored 'name' field.
	///
	/// Discovery traversals will call this method for every prim
	/// encountered in the scene.
	PREGEN_API virtual bool IsAsset(const pxr::UsdPrim& prim) const;

	/// Returns the asset definition for the given prim.
	///
	/// This method constructs an ExtAssetDefinition describing the discovered
	/// asset. Every definition must ultimately produce a unique identifier so
	/// that it can be unambiguously identified. The default implementation uses
	/// the following logic:
	///
	/// Asset info `name` must be authored, and is the primary token used when
	/// constructing a definition UID. If the `definitionPrefix` field in the
	/// supplied options is not empty it will be prepended to the name prior to
	/// constructing the UID. If asset info `version` is authored, or there is a
	/// version fallback mode active, the version will be appended to the UID.
	///
	/// A definition must have a valid SdfAssetPath so that the USD file for the
	/// asset can be located. If asset info `identifier` is authored it will be
	/// used as-is, otherwise the identifier fallback mode will attempt to
	/// construct a usable identifier by inspecting the reference and payload
	/// arcs present on the prim.
	///
	/// If no valid asset information can be determined, an invalid
	///	ExtAssetDefinition is returned.
	///
	/// Note that additional USD AssetInfo metadata is automatically copied
	/// into the definition metadata dictionary. This metadata is accessible
	/// when determining a final storage location for the eventual Engine
	/// assets via the active storage plugin, and can be used to build content
	/// folders based on arbitrary metadata (asset classifcations and/or
	/// categories for example).
	///
	/// Also note that metadata keys and values do not currently contribute to
	/// the automatically generated definition UID.
	PREGEN_API virtual ExtAssetDefinition
	GetAssetDefinition(const pxr::UsdPrim& prim) const;

	/// Manufacture a definition for the given root prim.
	///
	/// This is required when a discovery plugin returns false for IsAsset() for
	/// the first prim in the traversal, since there must be a singular root
	/// definition. It is possible for custom implementations to override this
	/// if they would rather handle root definitions in some special way.
	///
	/// By default the name for the definition will use the basename of the root
	/// layer identifier and a hash of the absolute path. If the root prim is
	/// not the pseudo root, a hash of the prim path also will be added to the
	/// name to avoid collisions. If fallback versioning is enabled, the version
	/// string will incorporate the paths and last modified timestamps of all
	/// layers in the local layer stack.
	PREGEN_API virtual ExtAssetDefinition
	CreateRootDefinition(const pxr::UsdPrim& rootPrim) const;

	/// Returns whether this prim and its descendants should be skipped during
	/// discovery. This can be used to discard prims that are not relevant to
	/// the discovery but would otherwise be considered, without requiring
	/// changes to the authored USD
	///
	/// Note that pruned prim paths are not currently factored into the
	/// permutation uid (hash) of the target. Changing the pruned state of
	/// otherwise identical scene description will not result in a different
	/// target.
	///
	/// Returning true prevents traversal of this prim and its subtree.
	PREGEN_API virtual bool PrunePrim(const pxr::UsdPrim& prim) const;

	/// Returns the currently active permutation for the given prim.
	///
	/// The default implementation constructs a PrimPermutation from
	/// the currently selected variants on the prims variant sets.
	///
	/// Variants that are excluded by the current DiscoveryOptions will not
	/// contribute to the permutation.
	PREGEN_API virtual PrimPermutation
	GetCurrentPrimPermutation(const pxr::UsdPrim& prim) const;

	/// Returns all possible permutations for the given prim.
	///
	/// The default implementation enumerates variant sets that do not
	/// currently have an authored selection and generates permutations
	/// from the Cartesian product of the available variants.
	///
	/// Variants that are excluded by the current DiscoveryOptions will not
	/// contribute to the permutations.
	///
	/// If no additional permutations exist, an empty vector is returned.
	///
	/// This method is only called when the DiscoveryMode of the current
	/// DiscoveryOptions is set to AllPermutations.
	PREGEN_API virtual std::vector<PrimPermutation>
	GetPrimPermutations(const pxr::UsdPrim& prim) const;

	/// Returns the content category for this prim.
	///
	/// Content categories can help scene consumers understand the types of
	/// prims and data present in the scene. Meshes, skeletons and materials
	/// for example each require a different kind of product.
	///
	/// Assembly-like assets that serve only as grouping structures for other
	/// assets often do not hold content of their own. The lack of content
	/// categories on a target is the primary mechanism for identifying
	/// organizational assets.
	PREGEN_API virtual pxr::TfToken
	GetContentCategoryForPrim(
		const pxr::UsdPrim& prim,
		const ExtAssetDefinition* defn) const;

	/// Returns a set of prim paths below the asset to be checked for overrides.
	///
	/// By default, all prims are considered when gathering overrides. Custom
	/// implementations may restrict this by returning a subset of prim paths
	/// to inspect. This function will be called only a single time per
	/// generated asset definition.
	///
	/// Return value semantics:
	/// - std::nullopt: check all prims (default behavior)
	/// - empty set: check no prims (disable override tracking for this asset)
	/// - non-empty set: only check the specified prim paths
	///
	PREGEN_API virtual std::optional<pxr::SdfPathSet>
	GetOverrideCandidatePrimPaths(
		const pxr::UsdPrim& assetPrim,
		const ExtAssetDefinition* defn) const;


	/// Get the current options.
	PREGEN_API const DiscoveryOptions& GetOptions() const;

private:

	DiscoveryOptions _options;

	pxr::SdfPath _initialPath;

	bool _excludeAllVariantSets = false;

	pxr::TfToken::HashSet _excludeVariantSets;

	mutable std::unique_ptr<struct internal::_VersionCache> _versionCache;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
