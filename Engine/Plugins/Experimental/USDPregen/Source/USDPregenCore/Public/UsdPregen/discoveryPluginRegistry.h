// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/discoveryOptions.h"
#include "UsdPregen/pregen.h"

#include <memory>
#include <functional>
#include <string>
#include <unordered_map>

#include "USDIncludesStart.h"
#include "pxr/base/tf/weakBase.h"
#include "USDIncludesEnd.h"

#define PREGEN_API USDPREGENCORE_API

PXR_NAMESPACE_OPEN_SCOPE

	template <typename T>
	class TfSingleton;

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

using DiscoveryPluginRefPtr = std::shared_ptr<class DiscoveryPlugin>;

/// \class DiscoveryPluginRegistry
///
/// Registry of discovery plugin factories.
///
/// DiscoveryPluginRegistry stores factory functions that construct
/// DiscoveryPlugin instances by name. The registry allows discovery
/// plugins to be registered by both C++ and Python implementations.
///
/// A typical workflow is:
///
/// 1. A plugin registers itself using RegisterFactory().
/// 2. The discovery system requests a plugin instance using Create().
/// 3. The returned plugin instance is used for the duration of the
///    discovery session.
///
/// Only a single plugin instance is typically active during a
/// discovery pass over a composed USD stage.
class DiscoveryPluginRegistry : public pxr::TfWeakBase
{
public:

	/// Factory function type used to construct plugins.
	///
	/// The returned DiscoveryPlugin instance will be wrapped in a
	/// shared pointer by the registry.
	using FactoryFn = std::function<DiscoveryPlugin*(const DiscoveryOptions&)>;

	/// Registers a discovery plugin factory under the given name.
	///
	/// If a factory with the same name already exists, the registration
	/// will be rejected.
	///
	/// \param name Unique name identifying the plugin implementation.
	/// \param fn Factory function used to construct the plugin.
	///
	/// \return True if the factory was successfully registered.
	PREGEN_API bool RegisterFactory(const std::string& name, FactoryFn fn);

	/// Creates a discovery plugin instance.
	///
	/// \param options Discovery options. `options.discoveryPluginName`
	/// selects the implementation; an empty name resolves to the
	/// built-in default discovery plugin.
	///
	/// \return A shared pointer to the constructed plugin instance,
	/// or nullptr if the plugin name is not registered.
	PREGEN_API DiscoveryPluginRefPtr Create(
		const DiscoveryOptions& options) const;

	/// Returns the global DiscoveryPluginRegistry instance.
	PREGEN_API static DiscoveryPluginRegistry& GetInstance();

private:
	friend class pxr::TfSingleton<DiscoveryPluginRegistry>;

	DiscoveryPluginRegistry() = default;
	~DiscoveryPluginRegistry() = default;

	std::unordered_map<std::string, FactoryFn> _registry;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
