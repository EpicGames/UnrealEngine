// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/usd/sdf/assetPath.h"
#include "pxr/base/vt/dictionary.h"
#include "USDIncludesEnd.h"

#include <optional>
#include <string>
#include <variant>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

/// \class ExtAssetDefinition
///
/// Describes a single external asset that may participate in the
/// asset discovery phase and eventual generation of associated
/// asset data.
///
/// An ExtAssetDefinition provides the minimal information required
/// to identify and locate an external asset:
///
///	 - A human-readable name
///	 - An optional version string
///	 - An identifier (typically a resolved asset path)
///	 - A unique identifier derived from the name and version, or explicitly
///	   constructed by a custom discovery plugin implementation.
///
/// Additional metadata may be attached using a VtDictionary. Metadata does not
/// affect the identity (unique ID) of the asset, but may carry auxiliary
/// information used during processing.
///
/// Instances are typically registered in the AssetDefinitionRegistry and
/// referenced by TargetDefinitionEntry and TargetData objects.
///
/// Various discovery plugin methods accept the active asset definition in
/// addition to the current USD prim, so the plugin does not need to track
/// the current definition.
class ExtAssetDefinition
{
public:

	/// Constructs an invalid asset definition.
	///
	/// A default-constructed instance does not represent a valid asset
	/// and will return false from IsValid().
	PREGEN_API ExtAssetDefinition();

	/// Constructs an asset definition using the default unique id scheme.
	///
	/// The unique id will be generated from the asset name and version. For
	/// example if the asset name is "Chair" and the version is "2", the
	/// resulting unique id would be "Chair_v2".
	///
	/// If a definition prefix was provided by the discovery options, it will
	/// be prepended, resulting in "Prefix_Chair_v2"
	PREGEN_API ExtAssetDefinition(const std::string& name,
								  const std::string& version,
								  const pxr::SdfAssetPath& identifier);

	/// Constructs an asset definition with a custom unique id.
	///
	/// This allows for additional state beyond name and version to contribute
	/// to the definition's unique id.
	///
	/// Note that the definition prefix is not automatically applied to the
	/// unique id, and must be applied by the caller by querying the current
	/// discovery options.
	PREGEN_API ExtAssetDefinition(const std::string& name,
								  const std::string& version,
								  const pxr::SdfAssetPath& identifier,
								  const std::string& customUniqueId);

	/// Constructs an asset definition with metadata.
	///
	/// The unique id will be generated from the asset name and version.
	///
	PREGEN_API ExtAssetDefinition(const std::string& name,
								  const std::string& version,
								  const pxr::SdfAssetPath& identifier,
								  const pxr::VtDictionary& metadata);

	/// Constructs an asset definition with both custom unique id and metadata.
	PREGEN_API ExtAssetDefinition(const std::string& name,
								  const std::string& version,
								  const pxr::SdfAssetPath& identifier,
								  const std::string& customUniqueId,
								  const pxr::VtDictionary& metadata);

	PREGEN_API ~ExtAssetDefinition() = default;

	PREGEN_API ExtAssetDefinition(const ExtAssetDefinition&) = default;
	PREGEN_API ExtAssetDefinition& operator=(const ExtAssetDefinition&) = default;

	PREGEN_API ExtAssetDefinition(ExtAssetDefinition&&) = default;
	PREGEN_API ExtAssetDefinition& operator=(ExtAssetDefinition&&) = default;

	PREGEN_API bool operator==(const ExtAssetDefinition& rhs) const;
	PREGEN_API bool operator!=(const ExtAssetDefinition& rhs) const;
	PREGEN_API bool operator<(const ExtAssetDefinition& rhs) const;

	/// Returns the asset name.
	///
	/// The name is an arbitrary identifier that describes the asset.
	PREGEN_API const std::string& GetName() const &;

	/// Returns the asset version string.
	///
	/// The version may be empty if the definition is unversioned.
	PREGEN_API const std::string& GetVersion() const &;

	/// Returns the identifier used to locate the asset.
	///
	/// Typically this is an SdfAssetPath referencing the USD file
	/// or entry point of the external asset.
	PREGEN_API const pxr::SdfAssetPath& GetIdentifier() const &;

	/// Returns the unique id for this asset definition.
	///
	/// The unique id is either automatically generated from the
	/// name and version or explicitly supplied at construction.
	PREGEN_API const std::string& GetUniqueId() const &;

	/// Returns additional metadata associated with the asset.
	///
	/// Metadata is optional and may contain arbitrary key-value
	/// pairs used by discovery or asset processing systems.
	PREGEN_API const std::optional<pxr::VtDictionary>& GetMetadata() const &;

	/// Returns true if metadata is present.
	PREGEN_API bool HasMetadata() const;

	/// Returns true if a custom unique id was supplied.
	PREGEN_API bool HasCustomUniqueId() const;

	/// Returns true if this asset definition represents a valid asset.
	///
	/// If \p reason is provided, it will be populated with a description
	/// of the validation failure.
	PREGEN_API bool IsValid(std::string* reason = nullptr) const;

	/// Convenience boolean conversion for validity checks.
	PREGEN_API explicit operator bool() const noexcept;

	/// Generates the default unique id from the asset name and version.
	PREGEN_API static std::string MakeDefaultUniqueId(
									  const std::string& name,
									  const std::string& version);

	/// Returns true if two asset definitions share the same defining fields.
	///
	/// This comparison includes differences in metadata.
	PREGEN_API static bool HasSameFields(const ExtAssetDefinition& defn1,
										 const ExtAssetDefinition& defn2);

private:

	/// Internal constructor used to unify construction paths.
	PREGEN_API ExtAssetDefinition(const std::string& name,
								  const std::string& version,
								  const pxr::SdfAssetPath& identifier,
								  const std::string& customUniqueId,
								  const std::optional<pxr::VtDictionary>& metadata);
	struct _DefaultUniqueId
	{
		std::string value;
	};

	struct _CustomUniqueId
	{
		std::string value;
	};

	using _UniqueId = std::variant<_DefaultUniqueId, _CustomUniqueId>;

	// The name of the asset. Must not be empty for the definition to be valid.
	std::string _name;

	// Optional version string for the asset.
	std::string _version;

	// Identifier used to resolve and open the asset entry point.
	// The asset is considered valid only if this identifier resolves.
	pxr::SdfAssetPath _identifier;

	// Unique identifier for the asset definition.
	// Either generated automatically or explicitly supplied.
	_UniqueId _uniqueId;

	// Optional metadata associated with the asset definition.
	std::optional<pxr::VtDictionary> _metadata = std::nullopt;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
