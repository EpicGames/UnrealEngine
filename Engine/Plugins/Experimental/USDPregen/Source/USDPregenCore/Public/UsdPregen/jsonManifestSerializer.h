// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "UsdPregen/manifestSerializer.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

#include <cstdint>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

class Manifest;

struct ManifestPayload;

class JsonManifestSerializer : public ManifestSerializer
{
public:

	PREGEN_API JsonManifestSerializer() = default;

	PREGEN_API virtual bool Serialize(const Manifest& manifest,
									  ManifestPayload& payload) const override;

	PREGEN_API virtual Manifest Deserialize(const ManifestPayload& payload) const override;

	PREGEN_API virtual std::string GetEncoding() const override;

	PREGEN_API virtual std::string GetFileExtension() const override;

	PREGEN_API static std::string GetSchemaName();

	PREGEN_API static std::int32_t GetSchemaVersion();

	PREGEN_API static std::string Encoding();

	PREGEN_API static std::string FileExtension();

	PREGEN_API static std::size_t MaxFileSize();
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK

