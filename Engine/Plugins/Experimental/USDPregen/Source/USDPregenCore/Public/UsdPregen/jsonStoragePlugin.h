// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "UsdPregen/storageOptions.h"
#include "UsdPregen/storagePlugin.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/token.h"
#include "USDIncludesEnd.h"

#include <filesystem>
#include <memory>
#include <string>
#include <vector>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

namespace JsonStoragePluginTokens
{
	extern PREGEN_API const pxr::TfToken manifestDirEnvvar;
}

class ExtAssetDefinition;
class TargetUid;

using JsonStoragePluginRefPtr = std::shared_ptr<class JsonStoragePlugin>;

/// \class JsonStoragePlugin
///
/// Filesystem-based JSON implementation of StoragePlugin.
///
/// This plugin stores manifest payloads as serialized JSON files under a
/// configurable directory. Each manifest file corresponds to a single
/// `TargetUid` and is written using a filename derived from that identifier.
///
/// The storage directory may be overridden via the environment variable
/// `USDPREGEN_DEFAULT_STORAGE_DIR`.
class JsonStoragePlugin : public StoragePlugin
{
public:

	/// Construct a filesystem-backed storage plugin.
	///
	PREGEN_API explicit JsonStoragePlugin(const StorageOptions& options = {});

	/// Load a manifest payload from the filesystem.
	///
	/// The manifest is resolved using the supplied TargetUid and the
	/// plugin's filename mapping policy.
	///
	/// Returns:
	///		Loaded	   - manifest file successfully read.
	///		NotLoaded  - no manifest file exists for the target.
	///		Error	   - file read or validation failure occurred.
	PREGEN_API
	ManifestLoadResult
	LoadManifestPayload(const TargetUid& targetUid) override;

	/// Store a manifest payload directly to the filesystem.
	///
	/// The payload will be written using the plugin's filename mapping
	/// strategy for the supplied TargetUid.
	///
	/// Returns:
	///		Saved	   - manifest file successfully written.
	///		NotSaved   - a manifest file already exists for the given target.
	///		Error	   - file write failure occurred.
	PREGEN_API
	ManifestSaveResult
	StoreManifestPayload(const TargetUid& targetUid,
		const ManifestPayload& payload) override;

	/// Always returns NotSaved as "storing" is already writing to disk for
	/// this plugin.
	PREGEN_API
	ManifestSaveResult
	PersistManifestPayload(const TargetUid& targetUid) override;

	/// Serialize an in-memory Manifest into a payload suitable for storage.
	///
	/// The default implementation uses the plugin's JSON serializer.
	PREGEN_API
	ManifestPayload
	SerializeManifest(const Manifest& manifest) override;

	/// Deserialize a manifest payload into an in-memory Manifest instance.
	///
	/// The payload encoding must match the format produced by
	/// SerializeManifest().
	PREGEN_API
	Manifest
	DeserializeManifestPayload(const ManifestPayload& payload) override;

	/// Generate a name for a UAsset created from a discovered asset.
	///
	/// The name may incorporate information from the asset definition
	/// hierarchy, node identifier, and Unreal asset type.
	PREGEN_API std::string
	GetNameForUAsset(const TargetUid& targetUid,
		             const std::vector<const ExtAssetDefinition*>& definitions,
		             const std::string& assetType) override;

	/// Generate a package sub-path for a UAsset created from a discovered asset.
	///
	/// The returned path determines where the asset will appear within the
	/// Unreal Engine Content Browser virtual filesystem.
	PREGEN_API std::string
	GetPackageSubPathForUAsset(const TargetUid& targetUid,
							   const std::vector<const ExtAssetDefinition*>& definitions,
		                       const std::string& assetType) override;

	/// Construct the filesystem path for the manifest of `targetUid`.
	PREGEN_API std::string
	GetPathForManifest(const TargetUid& targetUid) override;

private:
	/// Validate and optionally create the manifest directory.
	void _CheckOrCreateManifestDir(const std::string& manifestDir);

	/// Directory where manifest files are stored.
	std::string _manifestDir;

	/// True if the manifest directory is valid and usable.
	bool _manifestDirReady = false;

	/// ${PLACEHOLDER} template used by GetPackageSubPathForUAsset. Falls
	/// back to StoragePlugin::DefaultPackageSubPathTemplate() if empty.
	std::string _packageSubPathTemplate;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
