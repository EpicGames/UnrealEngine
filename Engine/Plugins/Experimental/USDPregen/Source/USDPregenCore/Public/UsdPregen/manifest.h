// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "UsdPregen/target.h"

#include <cstdint>
#include <memory>
#include <string>
#include <vector>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

class TargetData;
using TargetDataRefPtr = std::shared_ptr<TargetData>;

/// \struct Product
///
/// Describes a generated Unreal product produced for a target.
///
/// A Product entry records the Unreal asset that should be created as
/// part of materializing a target. Each entry captures the Unreal asset
/// path, the Unreal class used to construct the asset, and the USD prim
/// that produced the asset.
///
/// Multiple products may be associated with a single target depending
/// on how the source USD asset expands during import.
struct Product
{
	/// Unreal package path where the asset was created.
	std::string upackagePath;

	/// Unreal object type
	std::string uclass;

	/// Node identifier for the generated Unreal node associated with the asset.
	std::string unodeId;

	/// USD prim type responsible for producing this product.
	std::string usdPrimType;

	/// Absolute USD prim path of the source prim.
	std::string usdPrimPath;
};

/// \class Manifest
///
/// Describes the set of Unreal assets generated for a target.
///
/// A Manifest pairs a list of Product entries (the Unreal assets that should
/// be created or updated) with the originating TargetData that describes
/// the asset definition stack, dependencies, and per-info permutation ops
/// that produced the target. Manifests are typically serialized to disk by
/// a ManifestSerializer implementation and retrieved later by a
/// StoragePlugin when the corresponding target is requested again.
///
/// The manifest's target identifier is sourced from the attached TargetData
/// (see GetTargetUid()) - there is no separate uid stored on the manifest
/// itself, so the two cannot drift out of sync.
///
/// A manifest is considered valid when it has TargetData attached, the
/// TargetData is itself valid, and at least one product is present.
class Manifest
{
public:

	/// Constructs an empty manifest with no products and no attached target data.
	PREGEN_API Manifest() = default;

	/// Returns the target identifier sourced from the attached TargetData.
	///
	/// Returns an invalid TargetUid when no target data is attached.
	PREGEN_API TargetUid GetTargetUid() const;

	/// Adds a product entry to the manifest.
	///
	/// Products describe Unreal assets generated when this target is
	/// materialized.
	PREGEN_API void AddProduct(const Product& product);

	/// Returns the list of products associated with this manifest.
	PREGEN_API const std::vector<Product>& GetProducts() const;

	/// Attaches the originating target data to this manifest.
	///
	/// The target data carries the manifest's target identifier (via
	/// GetTargetUid) plus the asset definition stack, dependencies, and
	/// per-info permutation ops that produced this manifest. Storage
	/// plugins use it to serialize a snapshot of the originating target
	/// alongside the products.
	///
	/// May be set to a null pointer to clear any previously attached data.
	PREGEN_API void SetTargetData(const TargetDataRefPtr& targetData);

	/// Returns the target data attached to this manifest.
	///
	/// May return null if no target data was attached.
	PREGEN_API const TargetDataRefPtr& GetTargetData() const;

	/// Returns true if this manifest has valid target data attached and at
	/// least one product.
	PREGEN_API bool IsValid() const;

	/// Convenience boolean conversion that returns IsValid().
	PREGEN_API explicit operator bool() const noexcept;

private:

	/// Products generated when this target is materialized.
	std::vector<Product> _products;

	/// Originating target data. Required for the manifest to be valid.
	TargetDataRefPtr _targetData;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
