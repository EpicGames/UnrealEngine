// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "UsdPregen/manifestTypes.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

#include <string>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

class Manifest;

struct ManifestPayload;

/// \class ManifestSerializer
///
/// Abstract interface used to convert Manifest objects to and from a
/// serialized representation.
///
/// A ManifestSerializer defines the encoding used to persist a Manifest
/// as a byte payload and to reconstruct a Manifest instance from that
/// payload. Implementations provide both the serialization format
/// (for example JSON, protobuf, or other encodings) and metadata that
/// describes that format.
///
/// Serialized data is stored in a ManifestPayload structure, which
/// contains both the raw byte buffer and an encoding identifier string.
/// The encoding returned by GetEncoding() must match the value written
/// into ManifestPayload::encoding during serialization.
///
/// Storage systems typically persist ManifestPayload objects directly
/// without interpreting their contents. The serializer therefore forms
/// the boundary between the in-memory Manifest representation and the
/// encoded data stored on disk or transmitted between processes.
class ManifestSerializer
{
public:

	/// Virtual destructor.
	PREGEN_API virtual ~ManifestSerializer();

	/// Returns the file extension associated with this serializer.
	///
	/// This value typically corresponds to the format used when writing
	/// serialized manifests to disk (for example ".json").
	PREGEN_API virtual std::string GetFileExtension() const = 0;

	/// Returns the encoding identifier used by this serializer.
	///
	/// The returned string identifies the serialization format written
	/// into ManifestPayload::encoding (for example "application/json").
	PREGEN_API virtual std::string GetEncoding() const = 0;

	/// Serialize a Manifest into a payload suitable for storage.
	///
	/// Implementations must populate the provided ManifestPayload with
	/// the encoded representation of the manifest and set the payload's
	/// encoding field to the value returned by GetEncoding().
	///
	/// Returns true on successful serialization.
	PREGEN_API virtual bool Serialize(const Manifest& manifest,
		ManifestPayload& payload) const = 0;

	/// Deserialize a Manifest from a serialized payload.
	///
	/// Implementations must interpret the payload according to the
	/// encoding used during serialization and reconstruct the original
	/// Manifest object.
	PREGEN_API virtual Manifest Deserialize(const ManifestPayload& payload) const = 0;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
