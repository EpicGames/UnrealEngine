// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/diagnostic.h"
#include "USDIncludesEnd.h"

#include <cstddef>
#include <string>
#include <vector>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

/// Raw serialized manifest payload.
///
/// `data` contains the serialized manifest contents and `encoding`
/// identifies the serialization format (for example: application/json).
struct ManifestPayload
{
	std::string encoding;
	std::vector<std::uint8_t> data;
};

/// Status codes returned when requesting manifest data.
enum class ManifestLoadStatus
{
	/// Manifest was successfully located and loaded.
	Loaded,

	/// A manifest does not exist for the requested target.
	DoesNotExist,

	/// An error occurred while attempting to acquire the manifest.
	Error,
};


struct ManifestLoadResult
{
	ManifestLoadStatus status = ManifestLoadStatus::Error;
	ManifestPayload payload;
	std::string message;
};


/// Status codes returned when storing manifest data.
enum class ManifestSaveStatus
{
	/// Manifest was successfully written.
	Saved,

	/// Manifest was not written (implementation-specific decision).
	NotSaved,

	/// An error occurred while attempting to write the manifest.
	Error
};

struct ManifestSaveResult
{
	ManifestSaveStatus status = ManifestSaveStatus::Error;
	std::string message;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
