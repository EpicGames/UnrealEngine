// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/usd/sdf/listOp.h"
#include "pxr/usd/sdf/path.h"
#include "USDIncludesEnd.h"

#include <functional>
#include <memory>
#include <vector>
#include <set>
#include <string>

#define PREGEN_API USDPREGENCORE_API

PXR_NAMESPACE_OPEN_SCOPE

	class SdfPrimSpec;
	class UsdPrim;

	template <typename T>
	class SdfHandle;

	using SdfPrimSpecHandle = SdfHandle<SdfPrimSpec>;

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

using PermutationOpRefPtr = std::shared_ptr<class PermutationOp>;
using PermutationOpVector = std::vector<PermutationOpRefPtr>;

/// \class PermutationOp
///
/// Base class representing a single permutation operation.
///
/// A PermutationOp describes a modification that contributes to a
/// PrimPermutation. Implementations apply configuration changes to a prim,
/// such as selecting variants, applying schemas, or authoring inherit arcs.
///
/// Permutation operations can be applied directly to a live UsdPrim
/// or serialized into USD for later composition.
///
/// The configuration of operations applied to a prim (see \ref PrimPermutation)
/// must be hashable in order to contribute to a target's unique id.
class PermutationOp
{
public:

	PREGEN_API virtual ~PermutationOp();

	/// Applies the operation to the given prim.
	virtual void Apply(pxr::UsdPrim& prim) = 0;

	/// Returns a stable unique identifier for the operation.
	virtual std::string GetUniqueId() const = 0;

	/// Serializes the operation into the provided prim spec.
	virtual void Serialize(pxr::SdfPrimSpecHandle& primSpec) const = 0;

	/// Factory function used to reconstruct a PermutationOp from a serialized
	/// prim spec produced by an earlier call to Serialize().
	using FromSpecFn = std::function<
		PermutationOpRefPtr(const pxr::SdfPrimSpecHandle& primSpec)
	>;

	/// Registers a factory used to reconstruct ops authored with the given
	/// prim spec type name. Type names must match the value written by the
	/// associated subclass's Serialize() implementation.
	///
	/// Returns false if a factory for the type name was already registered.
	PREGEN_API static bool RegisterFactory(const std::string& typeName,
	                                       FromSpecFn fn);

	/// Reconstructs a PermutationOp from a serialized prim spec.
	///
	/// The prim spec's type name is used to look up a factory previously
	/// registered via RegisterFactory(). Returns nullptr if no factory is
	/// registered for the spec's type name, or if the spec is null.
	PREGEN_API static PermutationOpRefPtr CreateFromSpec(
	                                       const pxr::SdfPrimSpecHandle& primSpec);
};

using UsdVariantPermutationOpRefPtr =
	std::shared_ptr<class UsdVariantPermutationOp>;

/// \class UsdVariantPermutationOp
///
/// Permutation operation that selects a USD variant.
///
/// This operation represents a variant selection authored on a
/// prim's variant set.
class UsdVariantPermutationOp : public PermutationOp
{
public:

	/// Constructs a variant permutation operation.
	PREGEN_API UsdVariantPermutationOp(const std::string& variantSetName,
									   const std::string& variantName);

	/// Equivalent to calling the standard constructor; provided for symmetry
	/// with the other PermutationOp subclasses.
	PREGEN_API static PermutationOpRefPtr FromSerialized(
		const std::string& variantSetName,
		const std::string& variantName);

	/// Applies the variant selection to the prim.
	PREGEN_API virtual void Apply(pxr::UsdPrim& prim) final;

	/// Returns the unique identifier for this operation.
	PREGEN_API virtual std::string GetUniqueId() const final;

	/// Serializes parameters to the given prim spec.
	PREGEN_API virtual void Serialize(pxr::SdfPrimSpecHandle& primSpec) const final;

	/// Returns the variant set and variant name.
	PREGEN_API std::pair<std::string, std::string> GetVariantSelection() const;

	PREGEN_API bool operator==(const UsdVariantPermutationOp& rhs) const;
	PREGEN_API bool operator!=(const UsdVariantPermutationOp& rhs) const;

private:

	std::string _variantSetName;
	std::string _variantName;
};

using UsdInheritPermutationOpRefPtr =
	std::shared_ptr<class UsdInheritPermutationOp>;

/// \class UsdInheritPermutationOp
///
/// Permutation operation that authors a USD inherit arc.
class UsdInheritPermutationOp : public PermutationOp
{
public:

	PREGEN_API UsdInheritPermutationOp(
				   const pxr::SdfPath& pathToInherit,
				   pxr::SdfListOpType listOpType = pxr::SdfListOpTypePrepended);

	/// Equivalent to calling the standard constructor; provided for symmetry
	/// with the other PermutationOp subclasses.
	PREGEN_API static PermutationOpRefPtr FromSerialized(
		const pxr::SdfPath& pathToInherit,
		pxr::SdfListOpType listOpType = pxr::SdfListOpTypePrepended);

	/// Applies the inherit to the prim.
	PREGEN_API virtual void Apply(pxr::UsdPrim& prim) final;

	/// Returns the unique identifier for this operation.
	PREGEN_API virtual std::string GetUniqueId() const final;

	/// Serializes parameters to the given prim spec.
	PREGEN_API virtual void Serialize(pxr::SdfPrimSpecHandle& primSpec) const final;

	/// Returns the path that will be inherited.
	PREGEN_API const pxr::SdfPath& GetPathToInherit() const;

	/// Returns the list operation type used when authoring the inherit.
	PREGEN_API pxr::SdfListOpType GetListOpType() const;

private:

	const pxr::SdfPath _pathToInherit;
	pxr::SdfListOpType _listOpType;
};

using SchemaApplyPermutationOpRefPtr =
	std::shared_ptr<class SchemaApplyPermutationOp>;

/// \class SchemaApplyPermutationOp
///
/// Permutation operation that applies a USD API schema.
class SchemaApplyPermutationOp : public PermutationOp
{
public:

	/// Constructs an operation that applies the given schema.
	PREGEN_API SchemaApplyPermutationOp(const std::string& schemaName);

	/// Reconstructs a schema-apply permutation operation from serialized fields.
	///
	/// Equivalent to calling the standard constructor; provided for symmetry
	/// with the other PermutationOp subclasses.
	PREGEN_API static PermutationOpRefPtr FromSerialized(const std::string& schemaName);

	/// Applies the schema to the prim.
	PREGEN_API virtual void Apply(pxr::UsdPrim& prim) final;

	/// Returns the unique identifier for this operation.
	PREGEN_API virtual std::string GetUniqueId() const final;

	/// Serializes parameters to the given prim spec.
	PREGEN_API virtual void Serialize(pxr::SdfPrimSpecHandle& primSpec) const final;

	PREGEN_API std::string GetSchemaName() const;

private:

	const std::string _schemaName;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
