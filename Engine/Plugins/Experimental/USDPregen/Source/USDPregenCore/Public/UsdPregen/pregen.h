// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define PREGEN_NS ue::usdpregen

#define PREGEN_NAMESPACE_OPEN_SCOPE namespace PREGEN_NS {

#define PREGEN_NAMESPACE_CLOSE_SCOPE }

#define PREGEN_NAMESPACE_USING_DIRECTIVE using namespace PREGEN_NS;
