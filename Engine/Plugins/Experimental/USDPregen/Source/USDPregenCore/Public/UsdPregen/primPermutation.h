// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"
#include "UsdPregen/permutationOps.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/usd/sdf/path.h"
#include "USDIncludesEnd.h"

#include <memory>
#include <vector>
#include <set>
#include <string>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

using PrimPermutationRefPtr = std::shared_ptr<class PrimPermutation>;
using PrimPermutationConstRefPtr = std::shared_ptr<const class PrimPermutation>;

/// \class PrimPermutation
///
/// Represents a permutation of scene description for a prim.
///
/// A PrimPermutation describes a set of operations that modify how a prim
/// should be interpreted or composed during asset generation. These
/// operations typically correspond to authored configurations such as
/// variant selections, inheritance relationships, or schema application.
///
/// Each permutation is associated with a specific prim path and contains
/// an ordered set of PermutationOp objects that describe the modifications
/// required to realize that permutation.
///
/// PrimPermutation objects are typically produced by DiscoveryPlugin
/// implementations when enumerating possible asset configurations.
class PrimPermutation
{
public:
	/// Constructs an empty permutation
	PREGEN_API PrimPermutation();

	/// Constructs an empty permutation for the given prim path.
	PREGEN_API PrimPermutation(const pxr::SdfPath& path);

	/// Constructs a permutation with an initial set of operations.
	PREGEN_API PrimPermutation(const pxr::SdfPath& path,
							   const PermutationOpVector& ops);

	/// Returns the prim path associated with this permutation.
	PREGEN_API const pxr::SdfPath& GetPath() const &;

	/// Returns a stable unique identifier for this permutation.
	///
	/// The identifier is derived from the prim path and the set of
	/// permutation operations.
	PREGEN_API std::string GetUniqueId() const;

	/// Appends a permutation operation.
	///
	/// Returns false if an operation with the same unique identifier
	/// has already been added.
	PREGEN_API bool AppendOp(const PermutationOpRefPtr& op);

	/// Returns the ordered set of permutation operations.
	PREGEN_API const PermutationOpVector& GetOps() const &;

	/// Removes the specified operations from the permutation.
	PREGEN_API void RemoveOps(const PermutationOpVector& opsToRemove);

	/// Returns true if this permutation contains no operations.
	PREGEN_API bool IsEmpty() const;

	/// Returns true if this permutation consumes descendant scene
	/// description.
	///
	/// When enabled, all descendant scene description is treated as belonging
	/// to the asset owning this permutation, regardless of asset boundaries
	/// in descendant prims. This can needed if a schema modifies the ownership
	/// semantics of the asset hierarchy (the UE collapsing schema for example)
	PREGEN_API bool GetConsumesDescendants() const;

	/// Sets whether this permutation consumes descendant scene description.
	PREGEN_API void SetConsumesDescendants(bool value);

	/// Convenience boolean conversion for validity checks.
	PREGEN_API explicit operator bool() const noexcept;

private:

	const pxr::SdfPath _path;

	// Ordered list of permutation operations.
	PermutationOpVector _ops;

	// Set of operation identifiers used to prevent duplicates.
	std::set<std::string> _addedOps;

	// Whether this permutation causes descendant scene description to be
	// treated as part of the owning asset regardless of the descendant
	// asset hierarchy.
	bool _consumesDescendants = false;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
