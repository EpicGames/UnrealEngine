// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "UsdPregen/sceneTracker.h"

#include <memory>
#include <string>
#include <vector>

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

#define PREGEN_API USDPREGENCORE_API

PXR_NAMESPACE_OPEN_SCOPE

	class SdfPath;
	class UsdPrim;
	class UsdStage;

	template <typename T>
	class TfRefPtr;

	using UsdStageRefPtr = TfRefPtr<UsdStage>;

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

struct DiscoveryOptions;

/// \class SceneDiscovery
///
/// Convenience wrapper for executing a full USD stage traversal using a
/// SceneTracker instance.
///
/// While SceneTracker provides the core functionality for asset discovery and
/// permutation management, it expects the caller to perform the traversal
/// and correctly manage the TrackedPrim RAII objects returned by
/// SceneTracker::StartTrackingPrim.
///
/// SceneDiscovery encapsulates this pattern and performs a complete traversal of
/// the stage namespace, collecting discovered targets along the way.
///
/// Internally, SceneDiscovery mirrors the standard traversal pattern required by
/// SceneTracker and manages permutation exploration for each prim encountered.
class SceneDiscovery
{
public:

	/// Map from scene paths to the list of target identifiers discovered at
	/// that location.
	using ResultMap = std::unordered_map<pxr::SdfPath,
										 std::vector<class TargetUid>,
										 pxr::TfHash>;

	/// Constructs a traversal using the default tracker configuration.
	PREGEN_API SceneDiscovery(const pxr::UsdStageRefPtr& stage);

	/// Constructs a traversal with custom tracker options.
	PREGEN_API SceneDiscovery(const pxr::UsdStageRefPtr& stage,
						 const DiscoveryOptions& options);

	PREGEN_API SceneDiscovery(const SceneDiscovery&) = default;
	PREGEN_API SceneDiscovery& operator=(const SceneDiscovery&) = default;
	PREGEN_API SceneDiscovery(SceneDiscovery&&) = default;
	PREGEN_API SceneDiscovery& operator=(SceneDiscovery&&) = default;

	/// Returns the TargetData associated with a previously discovered target.
	///
	/// The provided identifier must originate from the results returned by
	/// TraverseAndFindTargets.
	PREGEN_API TargetDataRefPtr GetTargetData(const TargetUid& targetUid);

	/// Traverses the stage and discovers all targets.
	///
	/// The results map will be populated with scene paths mapped to the list
	/// of TargetUid identifiers discovered at each location.
	///
	/// Returns false if traversal encountered fatal tracking errors.
	PREGEN_API bool TraverseAndFindTargets(ResultMap& results);

	/// Saves the internal discovery data layer produced by the tracker.
	///
	/// This layer contains serialized target and definition data collected
	/// during traversal.
	PREGEN_API bool SaveDiscoveryData(const std::string& filename);

private:

	void _Traverse(const pxr::UsdPrim& prim);

	/// The stage being traversed.
	pxr::UsdStageRefPtr _stage;

	/// SceneTracker responsible for asset discovery and permutation management.
	SceneTrackerRefPtr _tracker;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
