// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"
#include "UsdPregen/discoveryOptions.h"
#include "UsdPregen/types.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/hash.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/usd/prim.h"
#include "USDIncludesEnd.h"

#include <cstddef>
#include <functional>
#include <memory>
#include <optional>
#include <string>
#include <unordered_map>
#include <vector>

#define PREGEN_API USDPREGENCORE_API

PXR_NAMESPACE_OPEN_SCOPE

	class PcpPrimIndex;
	class SdfPrimSpec;
	class UsdStage;

	template <typename T>
	class TfRefPtr;

	template <typename T>
	class SdfHandle;

	using SdfPrimSpecHandle = SdfHandle<SdfPrimSpec>;
	using UsdStageRefPtr = TfRefPtr<UsdStage>;

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

namespace internal
{
	class Item;
	class ItemScope;
	class LayerWriter;
	class PrimPermutationSet;
	using PrimPermutationSetPtr = std::unique_ptr<PrimPermutationSet>;
}

class ExtAssetDefinition;
class TargetUid;

using PrimPermutationRefPtr = std::shared_ptr<class PrimPermutation>;
using PrimPermutationConstRefPtr = std::shared_ptr<const class PrimPermutation>;
using TargetDataRefPtr = std::shared_ptr<class TargetData>;

using SceneTrackerRefPtr = std::shared_ptr<class SceneTracker>;
using SceneTrackerWeakPtr = std::weak_ptr<class SceneTracker>;

using DiscoveryPluginRefPtr = std::shared_ptr<class DiscoveryPlugin>;

/// \class TrackedPrim
///
/// RAII helper used by SceneTracker during scene traversal.
///
/// A TrackedPrim represents a prim currently being tracked by the SceneTracker.
/// It manages the lifecycle of asset discovery and permutation iteration
/// for that prim.
///
/// Instances are returned by SceneTracker::StartTrackingPrim and must remain alive
/// for the duration of processing for that prim. When the object is destroyed
/// it finalizes the associated tracker state and restores the tracker's
/// internal asset and permutation stacks.
///
/// Note: Direct use of TrackedPrim and SceneTracker in a custom traversal is
/// intended for advanced use cases where additional processing or bookkeeping
/// is required while walking the stage in a standalone context. If no custom
/// per-prim logic is needed, the built-in SceneDiscovery class is the
/// preferred option.
///
/// Care must be taken not to directly modify the compositional state of the
/// stage in a way that would interfere with the permutation processing.
///
/// Example recursive traversal function:
///
/// void
/// _Traverse(const pxr::UsdPrim& prim)
/// {
///     // Begin tracking for this prim. This establishes traversal state and
///     // returns a TrackedPrim handle used to query and drive permutations.
///  	TrackedPrim tracked = tracker->StartTrackingPrim(prim);
///
///     // At this point you can inspect the prim (properties, metadata, etc.)
///     // in its current composed state. Note that this may be called multiple
///     // times for the same prim as permutations are applied below.
///
///     // If the prim was pruned by DiscoveryPlugin::PrunePrim, the tracked
///     // handle will evaluate to false and the prim should be skipped entirely.
///  	if (!tracked)
///  	{
///  		return;
///  	}
///
///     // If this prim has unprocessed permutations, apply them one at a time.
///     // Each prepared permutation changes the composed state of the stage and
///     // then recursively re-enters this same prim under that new state.
///     //
///     // This means children are visited from inside those deeper permutation
///     // states first. Only after each recursive permutation traversal returns
///     // do we continue with the current composed state.
///     //
///     // Note that hierarchical permutations can quickly lead to a
///     // combinatorial explosion, resulting in thousands of recompositions.
///     // In practice, it’s best to restrict processing to permutations that
///     // are known to produce useful content. For example, the
///     // excludeVariantSets option can be used to skip unwanted variants.
///  	if (tracked.HasUnprocessedPermutations())
///  	{
///  		while(tracked.PrepareNextPermutation())
///  		{
///  			_Traverse(prim);
///  		}
///  		return;
///  	}
///
///     // Finally, continue traversal to the children using the predicate
///     // provided by the discovery plugin.
///  	UsdPrimSiblingRange children
///  		= prim.GetFilteredChildren(pxr::UsdTraverseInstanceProxies(
///  		    _tracker->GetOptions().traversalPredicate));
///
///  	for (pxr::UsdPrim childPrim : children)
///  	{
///  		_Traverse(childPrim);
///  	}
/// }
///
/// TrackedPrim behaves similarly to scoped guard types such as
/// std::lock_guard or UsdEditContext. It is non-copyable and may only
/// be move-constructed.
class TrackedPrim
{
public:

	/// Move construction is supported so the object can be returned
	/// directly from SceneTracker APIs.
	PREGEN_API TrackedPrim(TrackedPrim&&) noexcept;

	TrackedPrim& operator=(TrackedPrim&&) = delete;
	TrackedPrim(const TrackedPrim&) = delete;
	TrackedPrim& operator=(const TrackedPrim&) = delete;

	/// Finalizes tracking for the prim and restores tracker state.
	PREGEN_API ~TrackedPrim();

	/// Returns true if additional permutations remain to be processed.
	[[nodiscard]]
	PREGEN_API bool HasUnprocessedPermutations() const;

	/// Prepares the next permutation for evaluation.
	///
	/// Each successful call prepares tracker state so traversal can
	/// re-enter the same prim to process another permutation branch.
	///
	/// Returns false when no further permutations remain.
	[[nodiscard]]
	PREGEN_API bool PrepareNextPermutation();

	PREGEN_API explicit operator bool() const noexcept;

private:

	friend class SceneTracker;

	/// Internal lifecycle state of the tracked prim.
	///
	/// This state machine ensures that permutations are prepared and applied
	/// in a controlled manner while the traversal repeatedly re-enters the
	/// same prim. The phases roughly correspond to the following states:
	///
	/// Invalid
	///		Default state before tracking begins or after completion.
	///
	/// Started
	///		Tracking has begun and the prim has been registered with the
	///		tracker, but permutation state has not yet been prepared.
	///
	/// Ready
	///		The prim is ready for traversal. Either no permutations exist
	///		or the current permutation has already been applied.
	///
	/// Preparing
	///		A new permutation is currently being prepared and applied to the
	///		prim. After preparation completes, traversal will re-enter the
	///		prim with the updated composition state.
	///
	/// Completed
	///		All permutations have been processed and the prim's tracking
	///		lifecycle is complete.
	enum class _Phase : uint8_t
	{
		Invalid = 0,
		Started = 1,
		Ready = 2,
		Preparing = 3,
		Completed = 4
	};

	/// Local permutation iteration state for the tracked prim.
	///
	/// This is stored by value so that permutation iteration remains
	/// independent of the SceneTracker's internal traversal stack.
	struct _PermutationState
	{
		/// All permutations discovered for this prim.
		std::vector<PrimPermutationRefPtr> perms;

		/// Index of the permutation currently being evaluated.
		std::size_t currentIndex = 0;
	};

	TrackedPrim();

	TrackedPrim(const pxr::UsdPrim& prim,
				std::shared_ptr<class SceneTracker> tracker);

	bool _IsUsdPrimValid() const;

	/// The USD prim currently being tracked.
	pxr::UsdPrim _prim;

	/// Weak reference to the owning tracker.
	SceneTrackerWeakPtr _tracker;

	// The permutations and current index
	std::optional<_PermutationState> _permState = std::nullopt;

	mutable _Phase _phase = _Phase::Invalid;
};

/// \class SceneTracker
///
/// Central class responsible for discovering asset definitions and
/// generating targets during a USD scene traversal.
///
/// The SceneTracker inspects prims during traversal and uses a DiscoveryPlugin
/// to determine whether prims represent external assets. When assets are
/// discovered the tracker records their definitions, manages permutation
/// exploration, and constructs target representations describing the
/// resulting asset configuration.
///
/// SceneTracker maintains an internal stack of active assets and permutations
/// that mirrors the traversal order of the USD stage namespace.
///
/// As targets are discovered, the tracker invokes an optional callback
/// allowing clients to collect the generated TargetUid identifiers.
///
class SceneTracker : public std::enable_shared_from_this<SceneTracker>
{
public:

	using TargetCreatedCallback = std::function<
		void(const pxr::SdfPath& path, const TargetUid& targetUid)
	>;

	/// Creates a tracker using the default discovery plugin.
	PREGEN_API static SceneTrackerRefPtr
	Create(const pxr::UsdStageRefPtr& stage);

	/// Creates a tracker using the specified options.
	PREGEN_API static SceneTrackerRefPtr
	Create(const pxr::UsdStageRefPtr& stage, const DiscoveryOptions& options);

	/// Returns the tracker configuration.
	PREGEN_API const DiscoveryOptions& GetOptions() const;

	/// Returns the path from which scene discovery starts.
	///
	/// By default this will be the stage default prim, or the pseudo root if the
	/// stage does not have an authored default. A discovery plugin can nominate
	/// an initial path, which can be useful when the assets are known to exist
	/// in a subsection of the scene. The initial path can also be overridden
	/// in the discovery options.
	///
	/// Specifying a non-root initial path limits discovery to that subtree.
	/// Ancestor assets that would normally be discovered will be skipped and so
	/// this is not intended as a general-purpose filtering mechanism. To filter
	/// prims, implement the logic in the discovery plugin’s PrunePrim() method.
	PREGEN_API pxr::SdfPath GetInitialPath() const;

	/// Returns target data for the given target identifier.
	///
	/// The identifier must originate from the target created callback.
	PREGEN_API TargetDataRefPtr GetTargetData(const TargetUid& targetId) const;

	/// Returns true if errors were encountered during tracking.
	PREGEN_API bool HasErrors() const;

	/// Saves the internal Sdf data layer representation to disk.
	PREGEN_API bool SaveDataLayer(const std::string& filename) const;

	/// Begins tracking a prim during traversal.
	///
	/// The current implementation requires prims to be visited strictly in
	/// parent-to-child namespace order. Invoking this function on prims out of
	/// traversal order will lead to undefined behavior.
	[[nodiscard]]
	PREGEN_API TrackedPrim StartTrackingPrim(const pxr::UsdPrim& prim);

	/// Registers a callback invoked whenever a new target is created.
	PREGEN_API void SetTargetCreatedCallback(const TargetCreatedCallback& callback);

	/// Removes the active target created callback.
	PREGEN_API void RemoveTargetCreatedCallback();

private:

	friend class TrackedPrim;

	using _PurposeMask = std::uint8_t;

	enum class _PermutationApplyResult : uint8_t
	{
		InProgress,
		Completed,
		Invalid
	};

	/// Helper class for tracking error regions in namespace traversal.
	/// TODO: This currently invalidates the entire prim and its subtree, which
	/// works in general. However, if an error occurs within a specific
	/// permutation, there is no way to invalidate just that permutation while
	/// keeping the prim and its other permutations usable.
	class _ErrorMark
	{
		std::optional<std::size_t> _errDepth = std::nullopt;
		pxr::SdfPathSet _paths;

	public:

		void Set(const pxr::SdfPath& path)
		{
			_errDepth = path.GetPathElementCount();
			_paths.insert(path);
		}

		bool Check(const pxr::SdfPath& path)
		{
			if (!_errDepth.has_value())
			{
				return true;
			}
			else if (path.GetPathElementCount() > *_errDepth)
			{
				return false;
			}
			else
			{
				_errDepth = std::nullopt;
				return true;
			}
		}

		bool HasErrors() const {
			return !_paths.empty();
		}
	};

	struct _IdentifierEntry
	{
		pxr::SdfPath scenePath;
		types::int32 assetDepth;
	};

	// Bundles the state required to track and resolve permutations for a
	// specific prim. When visiting a prim, the tracker will create an instance
	// of this struct (mapped by prim path) if either GetCurrentPrimPermutation
	// or GetPrimPermutations on the discovery plugin return a non-empty result.
	//
	struct _PrimPermutationState
	{
		// Create a state and authored permutation, but no permutation set.
		_PrimPermutationState(const PrimPermutationConstRefPtr& authoredPerm);

		// Create permutation set for the given path, but no authored permutation.
		_PrimPermutationState(const pxr::SdfPath& path);

		// Return true if the state has a valid authored permutation.
		bool HasAuthoredPermutation() const;

		// Return true if the state has a valid permutation set.
		bool HasPermutationSet() const;

		// The originally composed permutation, required so that the tracker can
		// distinguish between a prim in its original and modified state.
		PrimPermutationConstRefPtr authoredPerm = nullptr;

		// The full set of possible permutations, as they are discovered.
		internal::PrimPermutationSetPtr permSet = nullptr;

		// Variant sets blocked during previous iterations.
		std::set<std::string> blockedVariantSets;
	};

	struct _PurposeInfo
	{
		pxr::TfToken purpose;
		bool hasAuthoredPurpose = false;
		_PurposeMask mask = 0;
		bool isIncluded = true;
	};

	// ------------------------------------------------------------------ //
	// Construction
	// ------------------------------------------------------------------ //

	SceneTracker(const pxr::UsdStageRefPtr& stage);

	SceneTracker(const pxr::UsdStageRefPtr& stage,
			const DiscoveryOptions& options,
			const DiscoveryPluginRefPtr& discoveryPlugin);

	void _SetInitialPath();

	// ------------------------------------------------------------------ //
	// Permutation helpers
	// ------------------------------------------------------------------ //

	void _AddPrimPermutations(const pxr::UsdPrim& prim);

	_PermutationApplyResult _ApplyPermutationOps(const PrimPermutationRefPtr& perm);

	std::vector<PrimPermutationRefPtr> _GetCurrentPermutations(const pxr::UsdPrim& prim);

	types::int32 _GetNextPermutationIndex() const;

	bool _HasUnprocessedPermutations(const pxr::UsdPrim& prim) const;

	_PrimPermutationState* _FindPrimPermutationState(const pxr::SdfPath& path);

	const _PrimPermutationState* _FindPrimPermutationState(const pxr::SdfPath& path) const;

	// ------------------------------------------------------------------ //
	// Asset stack management
	// ------------------------------------------------------------------ //

	const ExtAssetDefinition* _GetDefinitionFromPlugin(const pxr::UsdPrim& prim);

	const ExtAssetDefinition* _CreateRootDefinition(const pxr::UsdPrim& prim);

	void _CreateAndPushNewItem(const pxr::UsdPrim& prim,
		                       const ExtAssetDefinition* assetDefn);

	internal::Item* _GetCurrentItem();

	const internal::Item* _GetCurrentItem() const;

	bool _PushAsset(const pxr::UsdPrim& prim, const ExtAssetDefinition* assetDefn);

	void _PushPermutation(const PrimPermutationRefPtr& perm);

	void _PopAssetOrPermutation(const pxr::SdfPath& primPath);

	void _PopPurpose();

	// ------------------------------------------------------------------ //
	// Prim property helpers
	// ------------------------------------------------------------------ //

	//
	void _GatherPrimData(const pxr::UsdPrim& prim);

	types::int32 _GetOverrideDepthForSpec(
					 const pxr::PcpPrimIndex& index,
					 const pxr::SdfPrimSpecHandle& spec) const;

	// Blocks variant sets that have not yet been seen by the permutation state
	// for this prim. Each time a discovery plugin reports variant sets for a
	// prim in all-permutations mode, there are two scenarios:
	//
	// 1. The variant set is being reported for the first time, and its variants
	//    have not yet been visited (composed and traversed).
	//
	// 2. The variant set is already active, because the tracker has applied
	//    permutation ops that select it.
	//
	// For (1), we must block newly discovered variant sets so that their
	// selections are not incorporated into the scene description we are
	// inspecting. This effectively gives us a clean starting state. This is
	// especially important for nested variants, where multiple variants that
	// appear as siblings when queried via GetVariantSets may actually be
	// nested—blocking one can cause others to disappear entirely.
	//
	// For (2), it is important that we do not block these variant sets, as they
	// are already contributing to the composed scene state.
	//
	// The _PrimPermutationState structure maintains a list of variant sets it
	// has blocked, and only applies blocking to those not already in the list.
	void _BlockNewlyDiscoveredVariantSets(const pxr::UsdPrim& prim);

	// Returns true if the given scope represents the original authored
	// permutation state of the supplied stage (i.e. before any permutations
	// were applied)  This is only needed right now so that the LayerWriter is
	// able to author a mirror of the supplied scene hierarchy into the
	// discovery data layer (which is purely for diagnostic purposes).
	bool _ScopeMatchesAuthoredPermutation(const internal::ItemScope* itemScope) const;

	pxr::TfSpan<internal::Item*> _MakeItemView() const;

	std::string _MakeAssetHierarchyString() const;

	void _TrackPurpose(const pxr::UsdPrim& prim);

	static _PurposeMask _ToPurposeMask(const pxr::TfToken& purpose);

	// ------------------------------------------------------------------ //

	// The input stage to track
	pxr::UsdStageRefPtr _stage;

	// SceneTracker option configuration
	DiscoveryOptions _options;

	// The active discovery plugin.
	DiscoveryPluginRefPtr _discoveryPlugin = nullptr;

	// The starting path for tracking purposes. Because overrides will get
	// assigned using the asset definition hierarchy, a single initial path
	// associated with a definition is important so that stage-level overrides
	// can be associated with a definition uid. The ideal setup case is when
	// the discovery plugin is able to provide a definition for the initial
	// path. If not we must create one.
	pxr::SdfPath _initialPath;

	pxr::SdfPath _firstTrackedPath;

	// List of items that get pushed/popped as asset definitions are found and
	// associated with the stages namespace.
	std::vector<std::unique_ptr<internal::Item>> _itemStack;


	std::vector<_PurposeInfo> _purposeStack;
	_PurposeMask _allowedPurposeMask = 0;

	// Table used to accelerate resolved asset identifiers to their associated
	// position in the item stack.
	std::unordered_map<std::string, _IdentifierEntry> _identifierInfo;

	// Visited scene paths used to identify new paths.
	pxr::SdfPathSet _visitedScenePaths;

	// Permutation states, keyed by path.
	std::unordered_map<pxr::SdfPath,
		std::unique_ptr<_PrimPermutationState>,
					   pxr::TfHash> _permState;

	// The layer that stores the discovered definitions and targets.
	std::unique_ptr<internal::LayerWriter> _writer;

	// Mark the depth of the most recent path that produced an invalid asset
	// definition or a failed push. This is required to avoid tracking invalid
	// chunks of namespace.
	_ErrorMark _errorMark;

	// Reusable storage to construct a view of the current item stack.
	mutable std::vector<internal::Item*> _itemStore;

	// Invalid category names supplied by the discovery plugin, if any.
	std::unordered_set<pxr::TfToken, pxr::TfHash> _invalidCategories;

	// Callback invoked whenever a target becomes fully resolved. This callback
	// is currently invoked when the item or item scope is popped, in order to
	// ensure that overrides on descendant prims are fully captured. In the
	// future however, we will want the ability for users to opt out of override
	// collection, or provide a list of known overrides up-front, so that
	// targets can be generated without fully visiting all descendant prims.
	TargetCreatedCallback _targetCreatedCallback;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
