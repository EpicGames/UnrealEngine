// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "USDIncludesStart.h"
#include "USDIncludesEnd.h"

#include <string>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

/// \struct StorageOptions
///
/// Options forwarded to a storage plugin at construction time, via
/// StoragePluginRegistry::Create. Each plugin interprets fields it
/// understands and ignores the rest. Empty fields mean "use the
/// plugin's own default".
///
/// Example:
///     StorageOptions options;
///     options.storagePluginName = "json_storage";
///     options.manifestDir = "/tmp/manifests";
///     options.packageSubPathTemplate
///         = "assets/${DEFINITION_NAME}/${PERMUTATION_ID}";
///     auto storage = StoragePluginRegistry::GetInstance().Create(options);
struct StorageOptions
{
	/// Name of the storage plugin to use. An empty string will use the
	/// built-in JSON plugin ("json_storage"). The registry looks the
	/// name up in its factory map to construct the right plugin
	/// implementation.
	std::string storagePluginName;

	/// Directory where manifest files should be written.
	///
	/// Used by JsonStoragePlugin (filesystem-backed JSON storage).
	/// UObject-backed plugins ignore this since their manifests are
	/// UAssets, not files on disk.
	///
	/// Empty => plugin default (JsonStoragePlugin falls back to the
	/// USDPREGEN_DEFAULT_STORAGE_DIR environment variable, and then to
	/// "<HOME>/UE_UsdPregen_Manifests").
	std::string manifestDir;

	/// Template used by GetPackageSubPathForUAsset to build the
	/// content-browser sub-path for generated UAssets.
	///
	/// Recognized ${PLACEHOLDER} substitutions are documented on
	/// StoragePlugin::ResolvePackageSubPathTemplate.
	///
	/// Example: "assets/${DEFINITION_NAME}/${PERMUTATION_ID}" produces
	/// "assets/tree_02/2559017893".
	///
	/// Empty => plugin default
	/// (StoragePlugin::DefaultPackageSubPathTemplate).
	std::string packageSubPathTemplate;

	PREGEN_API bool operator==(const StorageOptions& rhs) const;

	PREGEN_API bool operator!=(const StorageOptions& rhs) const;

	PREGEN_API std::string DumpToString() const;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
