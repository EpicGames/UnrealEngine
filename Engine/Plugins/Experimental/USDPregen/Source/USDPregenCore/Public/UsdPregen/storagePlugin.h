// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "UsdPregen/storageOptions.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

#include <string>
#include <memory>
#include <unordered_map>
#include <vector>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

class ExtAssetDefinition;
class Manifest;
class TargetUid;

struct ManifestPayload;
struct ManifestLoadResult;
struct ManifestSaveResult;

using StoragePluginRefPtr = std::shared_ptr<class StoragePlugin>;

/// \class StoragePlugin
///
/// Abstract interface used to persist manifest data generated during
/// asset discovery and target generation.
///
/// Implementations define how serialized manifest payloads are stored
/// and retrieved for a given `TargetUid`. Storage backends may use the
/// filesystem, databases, remote services, or any other persistence
/// mechanism.
///
/// In addition to manifest persistence, this interface also controls how
/// generated Unreal assets are placed within the Unreal Engine Content
/// Browser (virtual filesystem). Implementations therefore define both:
///
///	  • the storage format and persistence strategy for manifest payloads
///	  • the naming and package layout for generated UAssets
///
/// StoragePlugin is intentionally stateless with respect to traversal.
/// Each request operates independently based solely on the supplied target
/// identifier.
class StoragePlugin
{
public:

	PREGEN_API StoragePlugin(const StorageOptions& options = {});

	PREGEN_API virtual ~StoragePlugin();

	/// Performs any one-time setup required to get the plugin into an
	/// operational state.
	///
	/// The base implementation performs no work and returns true.
	///
	/// \return Returns true on success, or false if initialization failed.
	PREGEN_API virtual bool Initialize();

	/// Performs cleanup prior to shutting down the plugin.
	///
	/// Plugins may override this method to release resources acquired during
	/// Initialize()
	///
	/// Shutdown() is intended to be paired with Initialize() as part of the
	/// plugin lifecycle and should typically be called explicitly by the
	/// owning system before destroying the plugin instance.
	///
	/// The base implementation performs no work and returns true.
	///
	/// \return Returns true on success, or false if clean shutdown failed.
	PREGEN_API virtual bool Shutdown();

	/// Load the serialized manifest payload associated with a target.
	///
	/// Returns:
	///		Loaded	     - manifest payload was successfully loaded
	///		DoesNotExist - a manifest does not exist for the target
	///		Error	     - an IO or validation failure occurred
	///
	/// When the status is `Loaded`, the returned result will contain the
	/// serialized payload.
	[[nodiscard]]
	PREGEN_API virtual ManifestLoadResult
	LoadManifestPayload(const TargetUid& targetUid) = 0;

	/// Store the serialized manifest payload for a target.
	///
	/// Implementations decide what "store" means: filesystem-backed plugins
	/// typically write the payload synchronously to disk, while UObject-based
	/// plugins may instead create an in-memory UAsset to be saved later.
	///
	/// Returns:
	///		Saved	 - manifest payload was successfully stored
	///		NotSaved - storage was intentionally skipped
	///		Error    - an IO or validation failure occurred
	[[nodiscard]]
	PREGEN_API virtual ManifestSaveResult
	StoreManifestPayload(const TargetUid& targetUid,
						 const ManifestPayload& payload) = 0;

	/// Make a previously-stored manifest durable (write it to its final
	/// destination) for implementations where there's a distinction between
	/// that and just "storing" the manifest.
	///
	/// Returns:
	///		Saved	 - manifest was made durable
	///		NotSaved - nothing to persist (already durable, or never stored)
	///		Error    - an IO or validation failure occurred
	[[nodiscard]]
	PREGEN_API virtual ManifestSaveResult
	PersistManifestPayload(const TargetUid& targetUid);

	/// Serialize an in-memory Manifest into a payload suitable for storage.
	///
	[[nodiscard]]
	PREGEN_API virtual ManifestPayload
	SerializeManifest(const Manifest& manifest) = 0;

	/// Deserialize a payload into an in-memory Manifest instance.
	///
	[[nodiscard]]
	PREGEN_API virtual Manifest
	DeserializeManifestPayload(const ManifestPayload& payload) = 0;

	/// Generate the name for a UAsset created from a discovered asset.
	///
	/// Implementations may derive names from the asset definition hierarchy,
	/// the asset identifier, and the requested Unreal asset type.
	PREGEN_API virtual std::string
	GetNameForUAsset(const TargetUid& targetUid,
					 const std::vector<const ExtAssetDefinition*>& definitions,
		             const std::string& assetType) = 0;

	/// Generate the package sub-path used when creating a UAsset.
	///
	/// The returned path is typically relative to a project content root
	/// and determines where the asset will appear within the Unreal Engine
	/// Content Browser.
	PREGEN_API virtual std::string
	GetPackageSubPathForUAsset(const TargetUid& targetUid,
		                       const std::vector<const ExtAssetDefinition*>& definitions,
							   const std::string& assetType) = 0;

	/// Determine the storage path for the manifest of `targetUid`.
	///
	/// The format of the returned path is plugin-specific (filesystem path,
	/// Unreal Engine package name, etc.) and is opaque to the framework.
	/// This method is the seam plugins override to control where manifests
	/// are stored.
	PREGEN_API virtual std::string
	GetPathForManifest(const TargetUid& targetUid) = 0;

	/// Default template used by ResolvePackageSubPathTemplate when no
	/// explicit template is provided. Matches the layout the built-in
	/// plugins used before the template became configurable, e.g.
	/// "assets/tree_02/versions/v01/permutations/2559017893".
	PREGEN_API static const std::string& DefaultPackageSubPathTemplate();

	/// Sentinel substituted for placeholder values that resolve to empty
	/// (or to an entirely non-alphanumeric string). Keeps templates like
	/// "versions/${DEFINITION_VERSION}/" producing "versions/_/" rather
	/// than orphan "versions/" segments when a definition has no version.
	PREGEN_API static const std::string& EmptyValueSentinel();

	/// Resolve `${PLACEHOLDER}` substitutions in `templateStr` and return
	/// the final package sub-path. Designed to be reused by all storage
	/// plugin implementations so the substitution semantics stay
	/// consistent.
	///
	/// Recognized placeholders (all wrapped in `${...}`):
	///   ${DEFINITION_NAME}    - definitions.back().GetName()
	///   ${DEFINITION_VERSION} - definitions.back().GetVersion()
	///   ${DEFINITION_UID}     - definitions.back().GetUniqueId()
	///   ${PERMUTATION_ID}     - targetUid.GetPermutationUid()
	///   ${ASSET_TYPE}         - the assetType parameter
	///   ${METADATA:KEY}       - looks up KEY in the leaf definition's
	///                           metadata VtDictionary (populated from
	///                           USD assetInfo, less the built-in keys).
	///                           Nested dicts are descended via colon-
	///                           separated paths, e.g.
	///                           ${METADATA:nested:subcategory}.
	///                           Non-scalar leaf values (dicts/arrays)
	///                           and missing keys both collapse to the
	///                           empty-value sentinel.
	///
	/// `extraSubstitutions` is consulted first and overrides the built-in
	/// placeholders. Unknown placeholders resolve to the empty value
	/// sentinel ("_").
	///
	/// Each substituted value is sanitized: alphanumeric and underscore
	/// pass through; every other character is replaced with `_`; an empty
	/// result becomes the sentinel ("_"). After substitution, runs of `/`
	/// are collapsed and a single trailing `/` is trimmed.
	///
	/// Example: template "assets/${DEFINITION_NAME}/${PERMUTATION_ID}",
	/// name "tree_02", permutation "2559017893" ->
	/// "assets/tree_02/2559017893".
	PREGEN_API static std::string
	ResolvePackageSubPathTemplate(
		const std::string& templateStr,
		const TargetUid& targetUid,
		const std::vector<const ExtAssetDefinition*>& definitions,
		const std::string& assetType,
		const std::unordered_map<std::string, std::string>& extraSubstitutions = {}
	);

	PREGEN_API const StorageOptions& GetOptions() const;

private:

	StorageOptions _options;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
