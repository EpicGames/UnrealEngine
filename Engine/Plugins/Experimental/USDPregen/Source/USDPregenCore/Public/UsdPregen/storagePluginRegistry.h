// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"
#include "UsdPregen/storageOptions.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/weakBase.h"
#include "USDIncludesEnd.h"

#include <functional>
#include <memory>
#include <string>
#include <unordered_map>

#define PREGEN_API USDPREGENCORE_API

PXR_NAMESPACE_OPEN_SCOPE

template <typename T>
class TfSingleton;

PXR_NAMESPACE_CLOSE_SCOPE

PREGEN_NAMESPACE_OPEN_SCOPE

using StoragePluginRefPtr = std::shared_ptr<class StoragePlugin>;

/// \class StoragePluginRegistry
///
/// Singleton registry used to manage StoragePlugin plugin factories.
///
/// Storage plugins define how manifest payloads are persisted and how
/// generated Unreal assets are organized within the Unreal Engine
/// Content Browser. The registry maintains a mapping between plugin
/// names and factory functions capable of constructing instances of
/// those plugins.
///
/// Plugins register themselves with the registry by providing a
/// factory function that creates a new StoragePlugin instance.
/// The registry can then construct plugin instances on demand.
///
/// Factory functions intentionally return raw pointers rather than
/// smart pointers. This allows plugin implementations written in
/// Python to expose factory functions callable from C++ while still
/// transferring ownership to a shared_ptr inside the registry.
///
/// StoragePluginRegistry is implemented as a TfSingleton and is
/// accessed through GetInstance().
class StoragePluginRegistry : public pxr::TfWeakBase
{
public:

	/// Factory function used to create storage plugin instances.
	///
	/// The returned pointer will be wrapped in a shared_ptr when the
	/// plugin instance is created. Implementations should allocate
	/// a new object on each invocation.
	///
	/// The `options` argument carries per-instance configuration
	/// (manifest directory, sub-path template, etc.). Factories may
	/// interpret only the fields they understand. Empty fields mean
	/// "use the plugin's own default".
	///
	/// A raw pointer return type is required to support plugin
	/// implementations authored in Python.
	using FactoryFn = std::function<StoragePlugin*(const StorageOptions& options)>;

	/// Register a storage plugin factory under a given name.
	///
	/// Returns true if the factory was successfully registered.
	/// If a factory with the same name already exists, registration
	/// will fail.
	PREGEN_API bool RegisterFactory(const std::string& name, FactoryFn fn);

	/// Create a storage plugin instance.
	///
	/// The registry will invoke the factory associated with
	/// `options.storagePluginName`, forwarding `options`, and return the
	/// resulting instance wrapped in a shared_ptr. An empty plugin
	/// name resolves to the built-in JSON plugin ("json_storage").
	///
	/// Returns nullptr if no plugin has been registered under the
	/// requested name.
	PREGEN_API StoragePluginRefPtr Create(
		const StorageOptions& options
	) const;

	/// Returns the global StoragePluginRegistry instance.
	PREGEN_API static StoragePluginRegistry& GetInstance();

private:

	friend class pxr::TfSingleton<StoragePluginRegistry>;

	StoragePluginRegistry();
	~StoragePluginRegistry() = default;

	std::unordered_map<std::string, FactoryFn> _registry;
};

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
