// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "UsdPregen/permutationOps.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/usd/sdf/layer.h"
#include "pxr/usd/sdf/path.h"
#include "USDIncludesEnd.h"

#include <cstddef>
#include <memory>
#include <ostream>
#include <string>
#include <vector>

#define PREGEN_API USDPREGENCORE_API

PREGEN_NAMESPACE_OPEN_SCOPE

class ExtAssetDefinition;

namespace internal
{

	class LayerWriter;
	class TargetDataBuilder;

}

/// \class TargetUid
///
/// Represents the unique identifier of a generated target.
///
/// A TargetUid identifies a target asset definition and optionally a
/// permutation of that definition. The definition UID identifies the base
/// asset definition, while the optional permutation UID identifies a specific
/// configuration or variant of that definition.
///
/// TargetUid objects are used as stable identifiers when tracking generated
/// targets and associated manifest entries.
class TargetUid
{
public:

	/// Constructs an invalid TargetUid.
	PREGEN_API TargetUid() = default;

	/// Constructs a TargetUid for a definition with no permutation.
	///
	/// \param definitionUid Unique identifier of the asset definition.
	PREGEN_API TargetUid(const std::string& definitionUid);

	/// Constructs a TargetUid with a definition and permutation identifier.
	///
	/// \param definitionUid Unique identifier of the asset definition.
	/// \param permutationUid Identifier of the specific permutation.
	PREGEN_API TargetUid(const std::string& definitionUid,
		                 const std::string& permutationUid);

	/// Returns the definition UID string.
	PREGEN_API std::string GetDefinitionUid() const;

	/// Returns the permutation UID string.
	///
	/// Returns an empty string if no permutation is present.
	PREGEN_API std::string GetPermutationUid() const;

	/// Returns true if this TargetUid contains a permutation identifier.
	PREGEN_API bool HasPermutationUid() const;

	/// Returns the UID as a string.
	PREGEN_API std::string GetString() const;

	/// Returns true if this TargetUid represents a valid identifier.
	PREGEN_API bool IsValid() const;

	/// Returns true if this TargetUid is valid.
	PREGEN_API explicit operator bool() const noexcept;

	PREGEN_API bool operator==(const TargetUid& rhs) const;

	PREGEN_API bool operator!=(const TargetUid& rhs) const;

	PREGEN_API bool operator<(const TargetUid& rhs) const;

	/// Converts this TargetUid into a valid USD prim name.
	///
	/// Encodes the identifier into a form suitable for use as
	/// a prim name within a USD stage.
	PREGEN_API static pxr::TfToken ConvertToUsdPrimName(const TargetUid& targetUid);

private:

	friend class internal::LayerWriter;

	pxr::TfToken _definitionUid;
	pxr::TfToken _permutationUid;
};

/// Stream output operator for debugging and logging.
PREGEN_API std::ostream& operator<<(std::ostream& os, const TargetUid& targetUid);


/// \class TargetDefinitionEntry
///
/// Describes a scene instance of an asset definition.
///
/// A TargetDefinitionEntry associates an asset definition with a specific location
/// in the USD scene graph. Multiple TargetDefinitionEntry objects may exist for a
/// single TargetUid if asset definitions are hierarchically nested in
/// namespace.
class TargetDefinitionEntry
{
public:

	/// Constructs an empty TargetDefinitionEntry.
	PREGEN_API TargetDefinitionEntry() = default;

	/// Constructs a TargetDefinitionEntry.
	///
	/// \param defnUid Unique identifier of the asset definition.
	/// \param scenePath Path of the prim in the scene where the target occurs.
	PREGEN_API TargetDefinitionEntry(const std::string& defnUid,
		                  const pxr::SdfPath& scenePath);

	/// Constructs a TargetDefinitionEntry with an associated list of permutation ops.
	///
	/// Used during target data construction to attach the permutation ops
	/// that produced this scope of the target.
	PREGEN_API TargetDefinitionEntry(const std::string& defnUid,
		                  const pxr::SdfPath& scenePath,
		                  PermutationOpVector permutationOps);

	/// Returns the resolved asset definition associated with this target.
	///
	/// May return nullptr if the definition cannot be resolved.
	PREGEN_API const ExtAssetDefinition* GetDefinition() const;

	/// Returns the scene path associated with this target instance.
	PREGEN_API const pxr::SdfPath& GetScenePath() const &;

	/// Returns the permutation ops applied at this scope of the target.
	///
	/// May be empty when no permutation ops are associated with this scope.
	PREGEN_API const PermutationOpVector& GetPermutationOps() const &;

private:

	std::string _defnUid;
	pxr::SdfPath _scenePath;
	PermutationOpVector _permutationOps;
};

using TargetDataRefPtr = std::shared_ptr<class TargetData>;
using TargetDataWeakPtr = std::weak_ptr<class TargetData>;

/// \class TargetData
///
/// Aggregates all information associated with a single TargetUid.
///
/// TargetData stores the set of TargetDefinitionEntry instances describing where a
/// target occurs in the scene, any dependencies between other targets,
/// and an optional permutation overlay layer that represents the computed
/// USD modifications for the target.
///
/// Instances of TargetData are typically produced during a discovery phase.
class TargetData
{
public:

	/// Constructs an invalid TargetData instance.
	PREGEN_API TargetData() = default;

	/// Constructs TargetData for the given TargetUid.
	PREGEN_API explicit TargetData(const TargetUid& targetUid);

	/// Returns true if this TargetData instance contains valid data.
	PREGEN_API bool IsValid() const;

	/// Convenience boolean conversion for validity checks.
	PREGEN_API explicit operator bool() const noexcept;

	/// Returns the unique identifier of this target.
	PREGEN_API TargetUid GetUniqueId() const;

	/// Returns the number of TargetDefinitionEntry entries associated with this target.
	PREGEN_API std::size_t NumDefinitionEntries() const;

	/// Returns the TargetDefinitionEntry at the given index.
	PREGEN_API const TargetDefinitionEntry& GetDefinitionEntry(std::size_t index) const;

	/// Returns the list of TargetDefinitionEntry entries associated with this target.
	PREGEN_API const std::vector<TargetDefinitionEntry>& GetDefinitionEntries() const &;

	/// Disallow access on rvalues to prevent dangling references.
	const std::vector<TargetDefinitionEntry>& GetDefinitionEntries() const && = delete;

	/// Returns a USD layer containing the permutation overlay.
	///
	/// May return null if no overlay layer is required.
	PREGEN_API pxr::SdfLayerRefPtr GetPermutationOverlay() const;

	/// Returns the list of target dependencies for this target.
	PREGEN_API const std::vector<TargetUid>& GetDependencies() const &;

	/// Returns the lists of encapsulated/unencapsulated definition scene paths.
	///
	/// This can be useful for identifying override sources and avoiding
	/// processing of fully encapsulated regions of namespace.
	PREGEN_API const pxr::SdfPathSet& GetEncapsulatedDefinitionPaths() const &;
	PREGEN_API const pxr::SdfPathSet& GetUnencapsulatedDefinitionPaths() const &;

private:

	friend class internal::LayerWriter;
	friend class internal::TargetDataBuilder;

	TargetUid _targetUid;
	std::vector<TargetDefinitionEntry> _definitionEntries;
	std::vector<TargetUid> _dependencies;
	pxr::SdfPathSet _encapsulatedDefinitions;
	pxr::SdfPathSet _unencapsulatedDefinitions;
	pxr::SdfLayerRefPtr _permOverlayLayer;
};

namespace internal
{

/// \class TargetDataBuilder
///
/// Helper used by manifest deserializers to construct a
/// populated TargetData from serialized fields. Friended by TargetData so
/// it can write to the otherwise private member fields without exposing
/// setters on the public TargetData API surface.
///
/// Despite being declared in this public header, TargetDataBuilder lives
/// in the internal:: namespace by design - downstream consumers should
/// treat it as an implementation detail of the manifest storage backends.
class TargetDataBuilder
{
public:
	PREGEN_API explicit TargetDataBuilder(const TargetUid& targetUid);

	PREGEN_API void AddInfo(TargetDefinitionEntry info);

	PREGEN_API void SetDependencies(std::vector<TargetUid> dependencies);

	PREGEN_API void SetEncapsulatedDefinitionPaths(pxr::SdfPathSet paths);

	PREGEN_API void SetUnencapsulatedDefinitionPaths(pxr::SdfPathSet paths);

	PREGEN_API TargetDataRefPtr Build();

private:
	TargetDataRefPtr _targetData;
};

} // namespace internal

PREGEN_NAMESPACE_CLOSE_SCOPE

#undef PREGEN_API

#endif // USE_USD_SDK
