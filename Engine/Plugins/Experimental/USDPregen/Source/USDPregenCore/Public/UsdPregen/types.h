// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UsdPregen/pregen.h"

#include <cstdint>

PREGEN_NAMESPACE_OPEN_SCOPE

namespace types {

typedef std::int32_t int32;
inline constexpr int32 InvalidIndex = -1;

} // namespace types

PREGEN_NAMESPACE_CLOSE_SCOPE
