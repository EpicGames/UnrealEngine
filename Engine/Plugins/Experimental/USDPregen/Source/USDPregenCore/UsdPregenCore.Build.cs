// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDPregenCore : ModuleRules
	{
		public USDPregenCore(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseRTTI = true;
			bLegalToDistributeObjectCode = true;
			bUseUnity = false;
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"UnrealUSDWrapper",
				}
			);

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}