// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenHttpWorker.h"
#include "USDPregenInterchangeModule.h"

#include "Engine/Engine.h"

#include "Async/Async.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "HAL/PlatformProcess.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Guid.h"
#include "Misc/OutputDevice.h"
#include "Misc/Parse.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Dom/JsonObject.h"

DEFINE_LOG_CATEGORY_STATIC(LogUSDPregen, Log, All);

#define LOCTEXT_NAMESPACE "FUSDPregenHttpWorkerModule"

IMPLEMENT_MODULE(FUSDPregenHttpWorkerModule, USDPregenHttpWorker)

namespace UE::USDPregenHttpWorker::Private
{
	// Parse snake_case string enum values that the controller emits into
	// the corresponding UENUM. Unknown values silently keep the existing
	// default on the option struct.

	void ParseDiscoveryMode(const FString& InValue, EPregenDiscoveryMode& OutValue)
	{
		if (InValue == TEXT("all_permutations"))
		{
			OutValue = EPregenDiscoveryMode::AllPermutations;
		}
		else if (InValue == TEXT("composed_permutation_only"))
		{
			OutValue = EPregenDiscoveryMode::ComposedPermutationOnly;
		}
	}

	void ParseIdentifierFallbackMode(const FString& InValue, EPregenIdentifierFallbackMode& OutValue)
	{
		if (InValue == TEXT("none"))
		{
			OutValue = EPregenIdentifierFallbackMode::None;
		}
		else if (InValue == TEXT("first_direct_reference_or_payload"))
		{
			OutValue = EPregenIdentifierFallbackMode::FirstDirectReferenceOrPayload;
		}
	}

	void ParseVersionFallbackMode(const FString& InValue, EPregenVersionFallbackMode& OutValue)
	{
		if (InValue == TEXT("none"))
		{
			OutValue = EPregenVersionFallbackMode::None;
		}
		else if (InValue == TEXT("layer_stack_files_and_timestamps"))
		{
			OutValue = EPregenVersionFallbackMode::LayerStackFilesAndTimestamps;
		}
		else if (InValue == TEXT("resolved_layer_stack_files_and_timestamps"))
		{
			OutValue = EPregenVersionFallbackMode::ResolvedLayerStackFilesAndTimestamps;
		}
	}

	void ParseStringArray(const TArray<TSharedPtr<FJsonValue>>* InValues, TArray<FString>& OutValues)
	{
		if (!InValues)
		{
			return;
		}
		OutValues.Reset(InValues->Num());
		for (const TSharedPtr<FJsonValue>& Entry : *InValues)
		{
			FString EntryStr;
			if (Entry.IsValid() && Entry->TryGetString(EntryStr))
			{
				OutValues.Add(MoveTemp(EntryStr));
			}
		}
	}

	void ParseDiscoveryOptions(const TSharedPtr<FJsonObject>& InObj, FPregenDiscoveryOptions& OutOptions)
	{
		if (!InObj.IsValid())
		{
			return;
		}

		InObj->TryGetStringField(TEXT("discovery_plugin_name"), OutOptions.DiscoveryPluginName);
		InObj->TryGetStringField(TEXT("definition_prefix"), OutOptions.DefinitionPrefix);
		InObj->TryGetStringField(TEXT("initial_path"), OutOptions.InitialPath);

		const TArray<TSharedPtr<FJsonValue>>* Purposes = nullptr;
		if (InObj->TryGetArrayField(TEXT("purposes"), Purposes))
		{
			ParseStringArray(Purposes, OutOptions.Purposes);
		}

		const TArray<TSharedPtr<FJsonValue>>* ExcludeVariantSets = nullptr;
		if (InObj->TryGetArrayField(TEXT("exclude_variant_sets"), ExcludeVariantSets))
		{
			ParseStringArray(ExcludeVariantSets, OutOptions.ExcludeVariantSets);
		}

		FString EnumStr;
		if (InObj->TryGetStringField(TEXT("discovery_mode"), EnumStr))
		{
			ParseDiscoveryMode(EnumStr, OutOptions.DiscoveryMode);
		}
		if (InObj->TryGetStringField(TEXT("asset_identifier_fallback"), EnumStr))
		{
			ParseIdentifierFallbackMode(EnumStr, OutOptions.AssetIdentifierFallback);
		}
		if (InObj->TryGetStringField(TEXT("asset_version_fallback"), EnumStr))
		{
			ParseVersionFallbackMode(EnumStr, OutOptions.AssetVersionFallback);
		}
	}

	void ParseStorageOptions(const TSharedPtr<FJsonObject>& InObj, FPregenStorageOptions& OutOptions)
	{
		if (!InObj.IsValid())
		{
			return;
		}

		InObj->TryGetStringField(TEXT("storage_plugin_name"), OutOptions.StoragePluginName);
		InObj->TryGetStringField(TEXT("manifest_dir"), OutOptions.ManifestDir);
		InObj->TryGetStringField(TEXT("package_sub_path_template"), OutOptions.PackageSubPathTemplate);
	}
}	 // namespace UE::USDPregenHttpWorker::Private

class FUSDPregenWorkerLog : public FOutputDevice
{
public:
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		Lines.Add(FString(V));
		if (Lines.Num() > 1000)
		{
			Lines.RemoveAt(0, Lines.Num() - 1000);
		}
	}

	TArray<FString> Consume()
	{
		TArray<FString> Copy = MoveTemp(Lines);
		Lines.Reset();
		return Copy;
	}

private:
	TArray<FString> Lines;
};

struct FUSDPregenWorkerState final
{
	TAtomic<bool> bShuttingDown{ false };

	// Track in-flight HTTP requests so ShutdownModule can unbind/cancel them.
	TArray<TSharedRef<IHttpRequest, ESPMode::ThreadSafe>> InFlightRequests;

	// Controller connection

	FString ControllerUrl;
	FString WorkerId;

	FString ControllerJobId;

	bool bMustShutdown = false;
	bool bShutdownNotificationSent = false;

	// Poll state

	float LongpollIntervalSeconds = 30.0;
	bool bPollInFlight = false;
	bool bBusyRunningTask = false;

	// Connection attempts

	int32 ConsecutiveConnectionFailures = 0;
	float RetrySeconds = 1.0f;
	FTSTicker::FDelegateHandle RetryHandle;


	// Current task

	FString CurrentTaskId;
	FString CurrentLeaseId;
	FString CurrentJobId;
	FString CurrentNodeId;
	FString CurrentTitle;

	FString CurrentFile1;
	bool bEnabled = false;

	// Optional pregen task fields surfaced by the controller
	FString CurrentPermutationLayerPath;
	FPregenDiscoveryOptions CurrentDiscoveryOptions;
	FPregenStorageOptions CurrentStorageOptions;

	// Heartbeat
	FTSTicker::FDelegateHandle HeartbeatHandle;

	// Log capture
	TUniquePtr<FUSDPregenWorkerLog> LogCapture;

	// Handles to prevent us from importing before commandline arguments are parsed
	FDelegateHandle EngineInitCompleteHandle;
	FDelegateHandle FirstEndFrameHandle;

	TArray<TSharedPtr<FJsonValue>> MakeLogValue()
	{
		TArray<FString> LogLines;
		if (LogCapture)
		{
			LogLines = LogCapture->Consume();
		}

		TArray<TSharedPtr<FJsonValue>> JsonLogs;
		for (const FString& LogLine : LogLines)
		{
			JsonLogs.Add(MakeShared<FJsonValueString>(LogLine));
		}

		return JsonLogs;
	}
};

namespace UE::USDPregenHttpWorker::Private
{
	static constexpr float HeartbeatIntervalSeconds = 10.0f;
	static constexpr float ConnectionRetrySeconds = 1.0f;
	static constexpr float NextTaskDelaySeconds = 0.2f;
	static constexpr float RequestExitDelaySeconds = 1.5f;

	using FState = FUSDPregenWorkerState;
	using FStatePtr = TSharedPtr<FState, ESPMode::ThreadSafe>;
	using FWeakStatePtr = TWeakPtr<FState, ESPMode::ThreadSafe>;

	bool IsShuttingDown(const FStatePtr& State)
	{
		return !State.IsValid() || State->bShuttingDown;
	}

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateTrackedRequest(const FStatePtr& State)
	{
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Req = FHttpModule::Get().CreateRequest();
		if (State.IsValid())
		{
			State->InFlightRequests.Add(Req);
		}
		return Req;
	}

	void UntrackRequest(const FStatePtr& State, const FHttpRequestPtr& Req)
	{
		if (!State.IsValid() || !Req.IsValid())
		{
			return;
		}

		State->InFlightRequests.RemoveAll(
			[&Req](const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& Candidate)
			{
				return Candidate == Req.ToSharedRef();
			});
	}

	TSharedPtr<FJsonObject> ConfigurePostRequest(
		const FStatePtr& State,
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Req,
		const FString& Endpoint)
	{
		Req->SetURL(State->ControllerUrl + TEXT("/") + Endpoint);
		Req->SetVerb(TEXT("POST"));
		Req->SetHeader(TEXT("Content-Type"), TEXT("application/json"));

		const TSharedPtr<FJsonObject> Obj = MakeShared<FJsonObject>();
		Obj->SetStringField(TEXT("worker_id"), State->WorkerId);
		Obj->SetStringField(TEXT("job_id"), State->ControllerJobId);
		Obj->SetStringField(TEXT("task_id"), State->CurrentTaskId);
		Obj->SetStringField(TEXT("lease_id"), State->CurrentLeaseId);

		return Obj;
	}

	void SetRequestContent(
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Req,
		TSharedPtr<FJsonObject> JsonObj)
	{
		if (!ensure(JsonObj.IsValid()))
		{
			return;
		}

		FString Body;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&Body);
		FJsonSerializer::Serialize(JsonObj.ToSharedRef(), Writer);
		Req->SetContentAsString(Body);
	}


	void StartLongPoll(const FStatePtr& State);
	void ScheduleNextTaskRequest(const FStatePtr& State, float DelaySeconds = 0.0f);
	void IssueNextTaskRequest(const FStatePtr& State);
	void ScheduleRetry(const FStatePtr& State);
	bool TickRetry(const FStatePtr& State, float DeltaTime);
	void ResetConnectionRetry(const FStatePtr& State);
	void BeginTask_OnGameThread(const FStatePtr& State, const TSharedPtr<FJsonObject>& TaskObj);
	void StartSingleImport_OnGameThread(const FStatePtr& State, const FString& Filename);
	void OnImportDone_OnGameThread(const FStatePtr& State, int32 ImportedCount, bool bSuccess, const FString& ErrorMsg);
	void StartHeartbeat(const FStatePtr& State);
	void StopHeartbeat(const FStatePtr& State);
	bool SendHeartbeat(const FStatePtr& State, float DeltaTime);
	void NotifyTaskDone(const FStatePtr& State, bool bSuccess, int32 ImportedCount, const FString& Message);
	void NotifyWorkerShutdown(const FStatePtr& State);
	void RequestWorkerShutdownAndExit(const FStatePtr& State);
	void ResetCurrentTask(const FStatePtr& State);

	void StartLongPoll(const FStatePtr& State)
	{
		if (IsShuttingDown(State) || State->bMustShutdown)
		{
			return;
		}

		if (State->EngineInitCompleteHandle.IsValid())
		{
			FCoreDelegates::OnFEngineLoopInitComplete.Remove(State->EngineInitCompleteHandle);
			State->EngineInitCompleteHandle.Reset();
		}
		if (State->FirstEndFrameHandle.IsValid())
		{
			FCoreDelegates::OnEndFrame.Remove(State->FirstEndFrameHandle);
			State->FirstEndFrameHandle.Reset();
		}

		IssueNextTaskRequest(State);
	}

	void ScheduleNextTaskRequest(const FStatePtr& State, float DelaySeconds)
	{
		if (IsShuttingDown(State))
		{
			return;
		}

		FWeakStatePtr WeakState = State;
		FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda([WeakState](float DeltaTime)
				{
					FStatePtr PinnedState = WeakState.Pin();
					if (!IsShuttingDown(PinnedState))
					{
						IssueNextTaskRequest(PinnedState);
					}

					return false;
				}),
			DelaySeconds);
	}

	void IssueNextTaskRequest(const FStatePtr& State)
	{
		if (IsShuttingDown(State) || State->bPollInFlight || State->bBusyRunningTask || State->bMustShutdown)
		{
			return;
		}

		State->bPollInFlight = true;

		const FString Url = FString::Printf(
			TEXT("%s/next_task?worker_id=%s&job_id=%s"),
			*State->ControllerUrl,
			*FGenericPlatformHttp::UrlEncode(State->WorkerId),
			*FGenericPlatformHttp::UrlEncode(State->ControllerJobId));

		const FStatePtr RequestState = State;
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Req = CreateTrackedRequest(RequestState);
		Req->SetURL(Url);
		Req->SetVerb(TEXT("GET"));

		// Add a little to the poll value so that the controller gets time to respond first.
		const float PollPadding = 5.0f;
		Req->SetTimeout(State->LongpollIntervalSeconds + PollPadding);

		FWeakStatePtr WeakState = RequestState;
		Req->OnProcessRequestComplete().BindLambda(
			[WeakState](FHttpRequestPtr CompletedReq, FHttpResponsePtr Resp, bool bOk)
			{
				FStatePtr PinnedState = WeakState.Pin();
				if (IsShuttingDown(PinnedState))
				{
					return;
				}

				UntrackRequest(PinnedState, CompletedReq);

				PinnedState->bPollInFlight = false;

				if (!bOk || !Resp.IsValid())
				{
					ScheduleRetry(PinnedState);
					return;
				}

				ResetConnectionRetry(PinnedState);

				const FString Body = Resp->GetContentAsString();

				TSharedPtr<FJsonObject> Root;
				const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Body);
				if (!FJsonSerializer::Deserialize(Reader, Root) || !Root.IsValid())
				{
					// TODO: Add a new "task_assignment_failed" endpoint to the controller
					// so that it can immediately reclaim the lease for the task it sent
					// to this worker, if any. Right now the task won't get reissued until
					// the lease expires.
					ScheduleNextTaskRequest(PinnedState, UE::USDPregenHttpWorker::Private::NextTaskDelaySeconds);
					return;
				}

				bool bControllerRequestedShutdown = false;
				Root->TryGetBoolField(TEXT("must_shutdown"), bControllerRequestedShutdown);
				if (bControllerRequestedShutdown)
				{
					RequestWorkerShutdownAndExit(PinnedState);
					return;
				}

				const TSharedPtr<FJsonObject>* TaskObjPtr = nullptr;
				if (!Root->TryGetObjectField(TEXT("task"), TaskObjPtr) || !TaskObjPtr || !TaskObjPtr->IsValid())
				{
					// "task": null currently indicates there is no work, and so
					//  we can just schedule another task request.
					ScheduleNextTaskRequest(PinnedState, UE::USDPregenHttpWorker::Private::NextTaskDelaySeconds);
					return;
				}

				PinnedState->bBusyRunningTask = true;

				FWeakStatePtr TaskWeakState = PinnedState;
				TSharedPtr<FJsonObject> TaskObj = *TaskObjPtr;
				AsyncTask(ENamedThreads::GameThread, [TaskWeakState, TaskObj]()
					{
						FStatePtr TaskState = TaskWeakState.Pin();
						if (IsShuttingDown(TaskState))
						{
							return;
						}

						BeginTask_OnGameThread(TaskState, TaskObj);
					});
			});

		Req->ProcessRequest();
	}

	bool TickRetry(const FStatePtr& State, float DeltaTime)
	{
		if (IsShuttingDown(State))
		{
			return false;
		}

		State->RetryHandle.Reset();
		IssueNextTaskRequest(State);
		return false;
	}

	void ScheduleRetry(const FStatePtr& State)
	{
		if (IsShuttingDown(State))
		{
			return;
		}

		State->ConsecutiveConnectionFailures++;
		State->RetrySeconds = FMath::Min(State->RetrySeconds * 2.0f, 60.0f);

		UE_LOGF(
			LogUSDPregen,
			Warning,
			"Task request failed - retrying in %.1f seconds.",
			State->RetrySeconds
		);

		// Replace any existing retry timer so only one retry is ever pending.
		// This prevents multiple overlapping retries and ensures the latest
		// backoff delay is used.
		if (State->RetryHandle.IsValid())
		{
			FTSTicker::GetCoreTicker().RemoveTicker(State->RetryHandle);
			State->RetryHandle.Reset();
		}

		// Schedule a one-shot retry after RetrySeconds.
		// TickRetry() will clear the handle and issue the next request.
		FWeakStatePtr WeakState = State;
		State->RetryHandle = FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda([WeakState](float DeltaTime)
				{
					FStatePtr PinnedState = WeakState.Pin();
					if (IsShuttingDown(PinnedState))
					{
						return false;
					}

					return TickRetry(PinnedState, DeltaTime);
				}),
			State->RetrySeconds);
	}

	void ResetConnectionRetry(const FStatePtr& State)
	{
		if (!State.IsValid())
		{
			return;
		}

		State->ConsecutiveConnectionFailures = 0;
		State->RetrySeconds = UE::USDPregenHttpWorker::Private::ConnectionRetrySeconds;
	}

	void BeginTask_OnGameThread(const FStatePtr& State, const TSharedPtr<FJsonObject>& TaskObj)
	{
		if (IsShuttingDown(State) || !TaskObj.IsValid())
		{
			return;
		}

		ResetCurrentTask(State);

		State->CurrentTaskId = TaskObj->GetStringField(TEXT("task_id"));
		State->CurrentLeaseId = TaskObj->GetStringField(TEXT("lease_id"));
		State->CurrentJobId = TaskObj->GetStringField(TEXT("job_id"));
		State->CurrentNodeId = TaskObj->GetStringField(TEXT("node_id"));
		State->CurrentTitle = TaskObj->GetStringField(TEXT("title"));

		State->CurrentFile1 = TaskObj->GetStringField(TEXT("file_1"));
		State->bEnabled = TaskObj->GetBoolField(TEXT("enabled"));

		TaskObj->TryGetStringField(TEXT("permutation_layer_path"), State->CurrentPermutationLayerPath);

		const TSharedPtr<FJsonObject>* DiscoveryOptionsObj = nullptr;
		if (TaskObj->TryGetObjectField(TEXT("discovery_options"), DiscoveryOptionsObj))
		{
			ParseDiscoveryOptions(*DiscoveryOptionsObj, State->CurrentDiscoveryOptions);
		}

		const TSharedPtr<FJsonObject>* StorageOptionsObj = nullptr;
		if (TaskObj->TryGetObjectField(TEXT("storage_options"), StorageOptionsObj))
		{
			ParseStorageOptions(*StorageOptionsObj, State->CurrentStorageOptions);
		}

		UE_LOGF(
			LogUSDPregen, 
			Log,
			"Starting task %ls (%ls)",
			*State->CurrentTaskId,
			*State->CurrentTitle);

		StartHeartbeat(State);

		if (!State->bEnabled)
		{
			const int32 ImportedCount = 0;
			const bool bSuccess = true;
			const FString DisabledMessage = TEXT("disabled");
			
			OnImportDone_OnGameThread(State, ImportedCount, bSuccess, DisabledMessage);
			return;
		}

		StartSingleImport_OnGameThread(State, State->CurrentFile1);
	}

	void StartSingleImport_OnGameThread(const FStatePtr& State, const FString& Filename)
	{
		if (IsShuttingDown(State))
		{
			return;
		}

		// Build the pregen import options from the controller-provided fields.
		// node_id is the TargetUid string. The optional fields default to empty
		// when the controller doesn't supply them.
		FPregenImportOptions ImportOptions;
		ImportOptions.SourceFilePath = Filename;
		ImportOptions.TargetUid = State->CurrentNodeId;
		ImportOptions.Title = State->CurrentTitle;
		ImportOptions.PermutationLayerPath = State->CurrentPermutationLayerPath;
		ImportOptions.DiscoveryOptions = State->CurrentDiscoveryOptions;
		ImportOptions.StorageOptions = State->CurrentStorageOptions;
		ImportOptions.bAutomated = true;
		ImportOptions.bAutoSavePackages = true;

		FWeakStatePtr WeakState = State;
		FUSDPregenInterchangeModule::ImportFile(
			ImportOptions,
			[WeakState](
				const FPregenImportOptions& CompletedOptions,
				bool bSuccess,
				const TArray<FString>& PackageFilePaths)
			{
				FStatePtr PinnedState = WeakState.Pin();
				if (IsShuttingDown(PinnedState))
				{
					return;
				}

				UE_LOGF(
					LogUSDPregen,
					Log,
					"Import for target '%ls' finished with (%d) packages (success=%ls)",
					*CompletedOptions.TargetUid,
					PackageFilePaths.Num(),
					bSuccess ? TEXT("true") : TEXT("false")
				);

				const int32 SavedCount = PackageFilePaths.Num();

				AsyncTask(ENamedThreads::GameThread, [WeakState, SavedCount, bSuccess]()
					{
						FStatePtr GameThreadState = WeakState.Pin();
						if (IsShuttingDown(GameThreadState))
						{
							return;
						}

						OnImportDone_OnGameThread(GameThreadState, SavedCount, bSuccess, TEXT(""));
					});
			});
	}

	void OnImportDone_OnGameThread(const FStatePtr& State, int32 ImportedCount, bool bSuccess, const FString& ErrorMsg)
	{
		if (IsShuttingDown(State))
		{
			return;
		}

		StopHeartbeat(State);
		NotifyTaskDone(State, bSuccess, ImportedCount, ErrorMsg);
		ResetCurrentTask(State);
		State->bBusyRunningTask = false;
		IssueNextTaskRequest(State);
	}

	void StartHeartbeat(const FStatePtr& State)
	{
		if (IsShuttingDown(State))
		{
			return;
		}

		if (!State->HeartbeatHandle.IsValid())
		{
			FWeakStatePtr WeakState = State;
			State->HeartbeatHandle = FTSTicker::GetCoreTicker().AddTicker(
				FTickerDelegate::CreateLambda([WeakState](float DeltaTime)
					{
						FStatePtr PinnedState = WeakState.Pin();
						if (IsShuttingDown(PinnedState))
						{
							return false;
						}

						return SendHeartbeat(PinnedState, DeltaTime);
					}),
				UE::USDPregenHttpWorker::Private::HeartbeatIntervalSeconds);
		}
	}

	void StopHeartbeat(const FStatePtr& State)
	{
		if (State.IsValid() && State->HeartbeatHandle.IsValid())
		{
			FTSTicker::GetCoreTicker().RemoveTicker(State->HeartbeatHandle);
			State->HeartbeatHandle.Reset();
		}
	}

	bool SendHeartbeat(const FStatePtr& State, float DeltaTime)
	{
		if (IsShuttingDown(State) || State->CurrentTaskId.IsEmpty())
		{
			return false;
		}

		const FStatePtr RequestState = State;
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Req = CreateTrackedRequest(RequestState);

		const TSharedPtr<FJsonObject> Obj = ConfigurePostRequest(RequestState, Req, TEXT("task_heartbeat"));

		Obj->SetArrayField(TEXT("log_tail"), RequestState->MakeLogValue());
		SetRequestContent(Req, Obj);

		FWeakStatePtr WeakState = RequestState;
		Req->OnProcessRequestComplete().BindLambda(
			[WeakState](FHttpRequestPtr CompletedReq, FHttpResponsePtr Resp, bool bOk)
			{
				FStatePtr PinnedState = WeakState.Pin();
				if (IsShuttingDown(PinnedState))
				{
					return;
				}

				UntrackRequest(PinnedState, CompletedReq);
			});

		Req->ProcessRequest();

		return true;
	}

	void NotifyTaskDone(const FStatePtr& State, bool bSuccess, int32 ImportedCount, const FString& Message)
	{
		if (!State.IsValid())
		{
			return;
		}

		const FStatePtr RequestState = State;
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Req = CreateTrackedRequest(RequestState);
		const TSharedPtr<FJsonObject> Obj = ConfigurePostRequest(RequestState, Req, TEXT("task_done"));

		Obj->SetStringField(TEXT("state"), bSuccess ? TEXT("succeeded") : TEXT("failed"));
		Obj->SetNumberField(TEXT("imported_count"), ImportedCount);
		Obj->SetStringField(TEXT("message"), Message);
		Obj->SetArrayField(TEXT("log_tail"), RequestState->MakeLogValue());

		SetRequestContent(Req, Obj);

		FWeakStatePtr WeakState = RequestState;
		Req->OnProcessRequestComplete().BindLambda(
			[WeakState](FHttpRequestPtr CompletedReq, FHttpResponsePtr Resp, bool bOk)
			{
				FStatePtr PinnedState = WeakState.Pin();
				if (!PinnedState.IsValid())
				{
					return;
				}

				UntrackRequest(PinnedState, CompletedReq);
			});

		Req->ProcessRequest();
	}

	void NotifyWorkerShutdown(const FStatePtr& State)
	{
		if (!State.IsValid() || State->WorkerId.IsEmpty() || State->bShutdownNotificationSent)
		{
			return;
		}

		State->bShutdownNotificationSent = true;

		const FStatePtr RequestState = State;
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Req = CreateTrackedRequest(RequestState);
		const TSharedPtr<FJsonObject> Obj = ConfigurePostRequest(RequestState, Req, TEXT("worker_shutdown"));
		SetRequestContent(Req, Obj);

		FWeakStatePtr WeakState = RequestState;
		Req->OnProcessRequestComplete().BindLambda(
			[WeakState](FHttpRequestPtr CompletedReq, FHttpResponsePtr Resp, bool bOk)
			{
				FStatePtr PinnedState = WeakState.Pin();
				if (!PinnedState.IsValid())
				{
					return;
				}

				UntrackRequest(PinnedState, CompletedReq);
			});

		Req->ProcessRequest();
	}

	void RequestWorkerShutdownAndExit(const FStatePtr& State)
	{
		if (IsShuttingDown(State))
		{
			return;
		}

		State->bMustShutdown = true;

		UE_LOGF(
			LogUSDPregen, 
			Log,
			"Worker shutdown requested by controller: WorkerId=%ls, JobId=%ls",
			*State->WorkerId,
			*State->ControllerJobId
		);

		StopHeartbeat(State);
		NotifyWorkerShutdown(State);

		// Give the worker_shutdown HTTP request a chance to actually 
		// be sent before we exit.
		FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda([](float DeltaTime)
				{
					const bool bForced = false;
					FPlatformMisc::RequestExit(bForced);
					return false;
				}),
			UE::USDPregenHttpWorker::Private::RequestExitDelaySeconds);
	}

	void ResetCurrentTask(const FStatePtr& State)
	{
		if (!State.IsValid())
		{
			return;
		}

		State->CurrentTaskId.Reset();
		State->CurrentLeaseId.Reset();
		State->CurrentJobId.Reset();
		State->CurrentNodeId.Reset();
		State->CurrentTitle.Reset();
		State->CurrentFile1.Reset();
		State->CurrentPermutationLayerPath.Reset();
		State->CurrentDiscoveryOptions = FPregenDiscoveryOptions{};
		State->CurrentStorageOptions = FPregenStorageOptions{};
	}
}

void FUSDPregenHttpWorkerModule::StartupModule()
{
	using namespace UE::USDPregenHttpWorker::Private;

	State = MakeShared<FState, ESPMode::ThreadSafe>();

	// Get the job controller URL from the commandline, which is typically added 
	// automatically by the job controller when it is launching workers.

	static constexpr TCHAR JobControllerArg[] = TEXT("UsdPregenJobControllerUrl=");
	if (FString Url; FParse::Value(FCommandLine::Get(), JobControllerArg, Url))
	{
		Url = Url.TrimStartAndEnd();
		if (Url.IsEmpty() ||
			!(Url.StartsWith(TEXT("http://")) || Url.StartsWith(TEXT("https://"))))
		{
			UE_LOGF(
				LogUSDPregen, 
				Error,
				"Invalid controller URL '%ls' (must start with http:// or https://)",
				*Url
			);

			State.Reset();
			return;
		}
		Url.RemoveFromEnd(TEXT("/"));
		State->ControllerUrl = Url;
	}
	else
	{
		UE_LOGF(
			LogUSDPregen, 
			Log, 
			"A job controller address/port was not provided - this host will not participate in worker-based"
			"import tasks (use -%lshttp://127.0.0.1:<PORT> to connect to a local controller)",
			JobControllerArg
		);
		State.Reset();
		return;
	}

	// Get the job id from the command line, if any. Controller launched workers must
	// have the job id specified in order to avoid processing work for an incompatible
	// job. A manually launched worker does not require a job id and can pick up tasks
	// from any job.
	static constexpr TCHAR JobIdArg[] = TEXT("UsdPregenJobControllerJobId=");
	FParse::Value(FCommandLine::Get(), JobIdArg, State->ControllerJobId);

	// Get the worker id from the command line. When the controller launches a worker
	// it will assign a unique worker id. If this worker was manually started however
	// we must create one.
	static constexpr TCHAR WorkerIdArg[] = TEXT("UsdPregenJobControllerWorkerId=");
	if (!FParse::Value(FCommandLine::Get(), WorkerIdArg, State->WorkerId) || State->WorkerId.IsEmpty())
	{
		State->WorkerId = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphens);
	}

	// Get the poll interval that the controller is using.
	static constexpr TCHAR LongpollIntervalArg[] = TEXT("UsdPregenJobControllerLongpollInterval=");
	if (FString RawValue; FParse::Value(FCommandLine::Get(), LongpollIntervalArg, RawValue))
	{
		if (RawValue.IsNumeric())
		{
			State->LongpollIntervalSeconds = FCString::Atof(*RawValue);
		}
		else
		{
			UE_LOGF(
				LogUSDPregen, 
				Warning,
				"Invalid float passed for arg %ls=%ls. Using default.", 
				LongpollIntervalArg, 
				*RawValue
			);
		}
	}

	State->LogCapture = MakeUnique<FUSDPregenWorkerLog>();
	GLog->AddOutputDevice(State->LogCapture.Get());

	UE_LOGF(
		LogUSDPregen, 
		Log, 
		"Http worker started: WorkerId=%ls, JobId=%ls, ControllerUrl=%ls",
		*State->WorkerId,
		State->ControllerJobId.IsEmpty() ? TEXT("<none>") : *State->ControllerJobId,
		*State->ControllerUrl
	);

	// We always defer starting the long-poll loop by at least one frame, so any
	// engine setup that runs during the first tick (most importantly,
	// UEngine::TickDeferredCommands flushing -ExecCmds CVars) has applied before
	// we dispatch a task. See the FirstEndFrameHandle comment on FStatePtr for
	// the full reasoning.
	//
	// If the engine is already up we can skip straight to the OnEndFrame wait;
	// otherwise we wait for OnFEngineLoopInitComplete first, then chain into
	// the OnEndFrame wait.
	auto SubscribeOnEndFrameOnce = [](const FStatePtr& InState)
	{
		if (IsShuttingDown(InState))
		{
			return;
		}
		FWeakStatePtr WeakState = InState;
		InState->FirstEndFrameHandle = FCoreDelegates::OnEndFrame.AddLambda([WeakState]()
			{
				FStatePtr PinnedState = WeakState.Pin();
				if (PinnedState.IsValid() && PinnedState->FirstEndFrameHandle.IsValid())
				{
					FCoreDelegates::OnEndFrame.Remove(PinnedState->FirstEndFrameHandle);
					PinnedState->FirstEndFrameHandle.Reset();
				}
				if (IsShuttingDown(PinnedState))
				{
					return;
				}
				StartLongPoll(PinnedState);
			});
	};

	if (GEngine)
	{
		SubscribeOnEndFrameOnce(State);
	}
	else
	{
		FWeakStatePtr WeakState = State;
		State->EngineInitCompleteHandle = FCoreDelegates::OnFEngineLoopInitComplete.AddLambda(
			[WeakState, SubscribeOnEndFrameOnce]()
			{
				FStatePtr PinnedState = WeakState.Pin();
				if (PinnedState.IsValid() && PinnedState->EngineInitCompleteHandle.IsValid())
				{
					FCoreDelegates::OnFEngineLoopInitComplete.Remove(PinnedState->EngineInitCompleteHandle);
					PinnedState->EngineInitCompleteHandle.Reset();
				}
				if (IsShuttingDown(PinnedState))
				{
					return;
				}
				SubscribeOnEndFrameOnce(PinnedState);
			});
	}
}

void FUSDPregenHttpWorkerModule::ShutdownModule()
{
	using namespace UE::USDPregenHttpWorker::Private;

	FStatePtr LocalState = State;
	if (!LocalState.IsValid())
	{
		return;
	}

	LocalState->bShuttingDown = true;

	if (LocalState->EngineInitCompleteHandle.IsValid())
	{
		FCoreDelegates::OnFEngineLoopInitComplete.Remove(LocalState->EngineInitCompleteHandle);
		LocalState->EngineInitCompleteHandle.Reset();
	}
	if (LocalState->FirstEndFrameHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(LocalState->FirstEndFrameHandle);
		LocalState->FirstEndFrameHandle.Reset();
	}

	StopHeartbeat(LocalState);

	if (LocalState->RetryHandle.IsValid())
	{
		FTSTicker::GetCoreTicker().RemoveTicker(LocalState->RetryHandle);
		LocalState->RetryHandle.Reset();
	}

	for (const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& Req : LocalState->InFlightRequests)
	{
		Req->OnProcessRequestComplete().Unbind();
		Req->CancelRequest();
	}
	LocalState->InFlightRequests.Empty();

	if (LocalState->LogCapture)
	{
		GLog->RemoveOutputDevice(LocalState->LogCapture.Get());
		LocalState->LogCapture.Reset();
	}

	State.Reset();
}

#undef LOCTEXT_NAMESPACE
