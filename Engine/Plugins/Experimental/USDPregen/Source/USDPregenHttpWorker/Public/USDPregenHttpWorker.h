// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "Containers/Ticker.h"
#include "Delegates/IDelegateInstance.h"

/**
 * Headless worker for UsdPregen-based imports.
 */
class FUSDPregenHttpWorkerModule final : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:

	TSharedPtr<struct FUSDPregenWorkerState, ESPMode::ThreadSafe> State;
};