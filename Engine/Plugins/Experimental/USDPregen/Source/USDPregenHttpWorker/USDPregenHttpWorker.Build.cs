// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class USDPregenHttpWorker : ModuleRules
{
	public USDPregenHttpWorker(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
		);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
		);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine"
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"HTTP",
				"HTTPServer",
				"Json",
				"JsonUtilities",
				"USDPregenInterchange"
			}
		);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{

			}
		);
	}
}
