// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenBlueprintLibrary.h"

#include "USDPregenInterchangeModule.h"

void UUSDPregenBlueprintLibrary::ImportFile(const FPregenImportOptions& ImportOptions, const FUSDPregenOnImportDoneDynamic& OnImportDone)
{
	// Wrap the dynamic delegate in a TFunction so the native ImportFile can fire it.
	// Copy the delegate into the lambda: the import is async, so the original ref
	// argument may be gone by the time the callback runs. Unbound delegates are
	// allowed (AutoCreateRefTerm / explicit default-constructed) and simply skip
	// the Execute call.
	FUSDPregenInterchangeModule::ImportFile(
		ImportOptions,
		[OnImportDoneCopy = OnImportDone](const FPregenImportOptions& CompletedOptions, bool bSuccess, const TArray<FString>& SavedPackageFilePaths)
		{
			if (OnImportDoneCopy.IsBound())
			{
				OnImportDoneCopy.Execute(CompletedOptions, bSuccess, SavedPackageFilePaths);
			}
		});
}
