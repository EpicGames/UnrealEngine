// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenContext.h"

#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "USDConversionUtils.h"
#include "UsdPregenWrappers/Target.h"
#include "UsdWrappers/UsdPrim.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDPregenContext)

FString UUSDPregenContext::MakeAssetNodeUid(const UE::FUsdPrim& Prim, const FString& Prefix, const FString& Suffix) const
{
#if USE_USD_SDK
	using namespace UE::Interchange::USD;

	if (SceneDiscovery)
	{
		UE::FSdfPath PrimPath = Prim.GetPrimPath();

		// Ancestor walk to find the owning pregen target
		const TArray<UE::UsdPregen::FTargetUid>* TargetUids = nullptr;
		int32 IntroDepth = 0;
		for (UE::FSdfPath SearchPath = PrimPath; !SearchPath.IsEmpty(); SearchPath = SearchPath.GetParentPath())
		{
			TargetUids = SceneDiscoveryResults.Find(SearchPath);
			if (TargetUids && !TargetUids->IsEmpty())
			{
				IntroDepth = SearchPath.GetPathElementCount();
				break;
			}
			TargetUids = nullptr;
		}

		if (TargetUids)
		{
			const UE::UsdPregen::FTargetUid& TargetUid = (*TargetUids)[0];
			UE::UsdPregen::FTargetData TargetData = SceneDiscovery->GetTargetData(TargetUid);

			if (TargetData.IsValid())
			{
				TArray<UE::UsdPregen::FTargetDefinitionEntry> Infos = TargetData.GetDefinitionEntries();

				if (!Infos.IsEmpty())
				{
					UE::FSdfPath ScenePath = Infos.Last().GetScenePath();

					// Remap: strip the scene-specific prefix (up to intro depth), replace with scene path
					UE::FSdfPath PrefixPath = PrimPath;
					while (PrefixPath.GetPathElementCount() > IntroDepth)
					{
						PrefixPath = PrefixPath.GetParentPath();
					}
					UE::FSdfPath RemappedPath = PrimPath.ReplacePrefix(PrefixPath, ScenePath);

					// Build UID with target UID embedded for uniqueness across different targets
					return MakeNodeUid(FString::Printf(
						TEXT("%s<%s>%s%s"),
						*Prefix,
						*TargetUid.GetString(),
						*RemappedPath.GetString(),
						*Suffix
					));
				}
			}
		}
	}
#endif

	return Super::MakeAssetNodeUid(Prim, Prefix, Suffix);
}
