// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenInterchangeModule.h"

#include "USDPregenContext.h"
#include "USDPregenSettings.h"
#include "UsdPregenWrappers/JsonStoragePlugin.h"
#include "UsdPregenWrappers/Manifest.h"
#include "UsdPregenWrappers/ManifestTypes.h"
#include "UsdPregenWrappers/SceneDiscovery.h"
#include "UsdPregenWrappers/StoragePlugin.h"
#include "UsdPregenWrappers/StoragePluginRegistry.h"
#include "UsdPregenWrappers/Target.h"

#include "UnrealUSDWrapper.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "InterchangeAssetImportData.h"
#include "InterchangeManager.h"
#include "InterchangeSceneImportAsset.h"
#include "InterchangeSourceData.h"
#include "InterchangeUsdTranslator.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#include "Animation/Skeleton.h"
#include "AssetCompilingManager.h"
#include "Engine/Engine.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Texture.h"
#include "FileHelpers.h"
#include "HAL/IConsoleManager.h"
#include "LevelSequence.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY_STATIC(LogUSDPregenInterchange, Log, All);

static FString GPregenDiscoveryPluginName;
static FAutoConsoleVariableRef CVarPregenDiscoveryPluginName(
	TEXT("USD.Pregen.DiscoveryPluginName"),
	GPregenDiscoveryPluginName,
	TEXT("Name of the pregen discovery plugin to use. Empty uses the built-in default.")
);

static FString GPregenStoragePluginName = TEXT("uobject_storage");
static FAutoConsoleVariableRef CVarPregenStoragePluginName(
	TEXT("USD.Pregen.StoragePluginName"),
	GPregenStoragePluginName,
	TEXT("Name of the pregen storage plugin to use for manifests.")
);

static bool GPregenAutoGenerateVersions = false;
static FAutoConsoleVariableRef CVarPregenAutoGenerateVersions(
	TEXT("USD.Pregen.AutoGenerateVersions"),
	GPregenAutoGenerateVersions,
	TEXT("If true, automatically generates version hashes from the layer stack when no version is authored.")
);

namespace UE::USDPregenInterchange::Private
{
	// Returns the AssetImportData for the given Object, falling back to its PreviewSkeletalMesh's
	// AssetImportData when the Object is a Skeleton or PhysicsAsset (which the SkeletalMesh
	// factory creates as side effects without ever stamping AssetImportData on them). The
	// SkeletalMesh's factory node walks up to the same pregen target via the ancestor walk in
	// OnAssetDoneNative, so its prim path is a valid substitute for routing the Skeleton /
	// PhysicsAsset products into the correct manifest.
	UInterchangeAssetImportData* GetImportDataFromObject(UObject* Object)
	{
		if (!Object)
		{
			return nullptr;
		}

		if (UInterchangeAssetImportData* ImportData = UInterchangeAssetImportData::GetFromObject(Object))
		{
			return ImportData;
		}

		// These assets don't have AssetImportData, so we need to climb through the PreviewMesh
		//
		// TODO: Maybe just add AssetImportData to these? We'll end up using the prim paths from other assets
		// by doing this...
		USkeletalMesh* PreviewMesh = nullptr;
		if (const USkeleton* Skeleton = Cast<USkeleton>(Object))
		{
			PreviewMesh = Skeleton->GetPreviewMesh();
		}
		else if (const UPhysicsAsset* PhysicsAsset = Cast<UPhysicsAsset>(Object))
		{
			PreviewMesh = PhysicsAsset->GetPreviewMesh();
		}

		if (PreviewMesh)
		{
			return UInterchangeAssetImportData::GetFromObject(PreviewMesh);
		}

		return nullptr;
	}

	// RAII guard pairing StoragePlugin::Initialize with Shutdown.
	//
	// Held via TSharedRef so the async OnSceneImportDoneNative lambda capture is
	// the natural owner: Shutdown runs once, after the lambda is destroyed and
	// all manifest ops have completed. Early-return failure paths in ImportFile
	// also drop the ref and trigger Shutdown - unless Initialize itself failed,
	// in which case the bInitialized flag suppresses it.
	struct FScopedStorageInit
	{
		UE::UsdPregen::FStoragePlugin Storage;
		bool bInitialized = false;

		explicit FScopedStorageInit(UE::UsdPregen::FStoragePlugin InStorage)
			: Storage(MoveTemp(InStorage))
		{
		}

		bool Initialize()
		{
			bInitialized = Storage.Initialize();
			return bInitialized;
		}

		~FScopedStorageInit()
		{
			if (bInitialized)
			{
				Storage.Shutdown();
			}
		}

		FScopedStorageInit(const FScopedStorageInit&) = delete;
		FScopedStorageInit& operator=(const FScopedStorageInit&) = delete;
	};

	// Extracts the USD prim path from an imported asset by reading the prim path
	// attribute from its factory node (Interchange automatically moves user attributes
	// from translated nodes to factory nodes).
	FString GetPrimPathFromAsset(UObject* Asset)
	{
		UInterchangeAssetImportData* ImportData = GetImportDataFromObject(Asset);
		if (!ImportData)
		{
			return FString();
		}

		const UInterchangeBaseNode* FactoryNode = ImportData->GetStoredFactoryNode(ImportData->NodeUniqueID);
		if (!FactoryNode)
		{
			return FString();
		}

		// DuplicateAllUserDefinedAttribute prefixes keys with the source node's display label when
		// copying to factory nodes (e.g. "MeshNodeName._USDPrimPath_"). For some asset types
		// like collapsed meshes we may have multiple prim paths from different translated nodes.
		// We always want the shortest (rootmost) path
		TArray<FInterchangeUserDefinedAttributeInfo> AttributeInfos = UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttributeInfos(FactoryNode);
		FString ShortestPrimPath;
		for (const FInterchangeUserDefinedAttributeInfo& Info : AttributeInfos)
		{
			if (!Info.Name.Contains(UE::Interchange::USD::PrimPathAttributeKeyString))
			{
				continue;
			}

			FString CandidatePath;
			TOptional<FString> PayloadKey;
			if (UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(FactoryNode, Info.Name, CandidatePath, PayloadKey))
			{
				if (!CandidatePath.IsEmpty()
					&& (ShortestPrimPath.IsEmpty() || CandidatePath.Len() < ShortestPrimPath.Len()))
				{
					ShortestPrimPath = CandidatePath;
				}
			}
		}
		return ShortestPrimPath;
	}

	FPregenDiscoveryOptions MakeDiscoveryOptions(const FPregenImportOptions& ImportOptions)
	{
		FPregenDiscoveryOptions Options = ImportOptions.DiscoveryOptions;
		Options.DiscoveryMode = EPregenDiscoveryMode::ComposedPermutationOnly;
		Options.AssetIdentifierFallback = EPregenIdentifierFallbackMode::FirstDirectReferenceOrPayload;

		if (Options.DiscoveryPluginName.IsEmpty())
		{
			Options.DiscoveryPluginName = GPregenDiscoveryPluginName;
		}

		if (GPregenAutoGenerateVersions && Options.AssetVersionFallback == EPregenVersionFallbackMode::None)
		{
			Options.AssetVersionFallback = EPregenVersionFallbackMode::ResolvedLayerStackFilesAndTimestamps;
		}

		return Options;
	}

	FPregenStorageOptions MakeStorageOptions(const FPregenImportOptions& ImportOptions)
	{
		FPregenStorageOptions Options = ImportOptions.StorageOptions;

		if (Options.StoragePluginName.IsEmpty())
		{
			Options.StoragePluginName = GPregenStoragePluginName;
		}

		if (Options.PackageSubPathTemplate.IsEmpty())
		{
			Options.PackageSubPathTemplate = GetDefault<UUSDPregenSettings>()->PackageSubPathTemplate;
		}

		return Options;
	}

	// Apply the import's permutation overlay (if any) to Stage by sublayering the
	// overlay onto the stage's session layer. Session-layer opinions compose
	// stronger than anything in the root layer's stack.
	bool ApplyPermutationOverlay(UE::FUsdStage& Stage, const FString& PermutationLayerPath)
	{
		if (PermutationLayerPath.IsEmpty())
		{
			return true;
		}

		UE::FSdfLayer SessionLayer = Stage.GetSessionLayer();
		if (!SessionLayer)
		{
			UE_LOG(LogUSDPregenInterchange, Error, TEXT("Stage has no session layer; cannot apply permutation overlay '%s'"), *PermutationLayerPath);
			return false;
		}

		SessionLayer.InsertSubLayerPath(PermutationLayerPath);
		return true;
	}
}	 // namespace UE::USDPregenInterchange::Private

void FUSDPregenInterchangeModule::StartupModule()
{
}

void FUSDPregenInterchangeModule::ShutdownModule()
{
}

void FUSDPregenInterchangeModule::ImportFile(
	const FPregenImportOptions& ImportOptions,
	FOnImportDone OnImportDone)
{
	using namespace UE::USDPregenInterchange::Private;

	auto NotifyFailedImport = [&ImportOptions, &OnImportDone]()
		{
			if (OnImportDone)
			{
				const bool bSuccess = false;
				const static TArray<FString> SavedPackageFilePaths;
				OnImportDone(ImportOptions, bSuccess, SavedPackageFilePaths);
			}
		};

	const FString& FilePath = ImportOptions.SourceFilePath;
	if (FilePath.IsEmpty())
	{
		UE_LOG(LogUSDPregenInterchange, Error, TEXT("Pregen import has no source file path"));

		NotifyFailedImport();

		return;
	}

	if (!ImportOptions.Title.IsEmpty())
	{
		UE_LOG(LogUSDPregenInterchange, Log, TEXT("Pregen import: %s (%s)"), *ImportOptions.Title, *FilePath);
	}

	const TArray<UE::FUsdStage> StagesBeforeOpen = UnrealUSDWrapper::GetAllStagesFromCache();

	const bool bUseStageCache = false;
	const bool bForceReloadLayersFromDisk = true;
	const EUsdInitialLoadSet InitialLoadSet = EUsdInitialLoadSet::LoadAll;
	UE::FUsdStage EntryStage = UnrealUSDWrapper::OpenStage(*FilePath, InitialLoadSet, bUseStageCache, bForceReloadLayersFromDisk);
	if (!EntryStage)
	{
		UE_LOG(LogUSDPregenInterchange, Error, TEXT("Failed to open USD stage: %s"), *FilePath);

		NotifyFailedImport();

		return;
	}

	// If the stage was already in the cache before our OpenStage, we're reusing
	// someone else's stage which may have unrelated session-layer mods. We can't fix
	// this without breaking that other consumer, so just warn and continue.
	if (StagesBeforeOpen.Contains(EntryStage))
	{
		UE_LOG(
			LogUSDPregenInterchange,
			Warning,
			TEXT("Pregen import: stage for '%s' was already opened in the USD utils stage cache, and may be affected as the layers were reloaded from disk."),
			*FilePath
		);
	}

	if (!ApplyPermutationOverlay(EntryStage, ImportOptions.PermutationLayerPath))
	{
		NotifyFailedImport();

		return;
	}

	// Discovery
	FPregenDiscoveryOptions DiscoveryOptions = MakeDiscoveryOptions(ImportOptions);
	UE::UsdPregen::FSceneDiscovery SceneDiscovery{EntryStage, DiscoveryOptions};
	UE::UsdPregen::FSceneDiscovery::ResultMap SceneDiscoveryResults;
	if (!SceneDiscovery.TraverseAndFindTargets(SceneDiscoveryResults))
	{
		UE_LOG(LogUSDPregenInterchange, Error, TEXT("Pregen discovery/traversal failed for: %s"), *FilePath);
		
		NotifyFailedImport();

		return;
	}

	UE_LOG(
		LogUSDPregenInterchange,
		Log,
		TEXT("Pregen discovery found %d prim(s) with targets"),
		SceneDiscoveryResults.Num()
	);

	FPregenStorageOptions StorageOptions = MakeStorageOptions(ImportOptions);
	UE::UsdPregen::FStoragePlugin Storage = UE::UsdPregen::FStoragePluginRegistry::GetInstance().Create(StorageOptions);
	if (!Storage.IsValid())
	{
		UE_LOG(LogUSDPregenInterchange, Error, TEXT("Failed to create storage plugin: %s"), *StorageOptions.StoragePluginName);

		NotifyFailedImport();

		return;
	}

	TSharedRef<FScopedStorageInit> ScopedStorage = MakeShared<FScopedStorageInit>(MoveTemp(Storage));
	if (!ScopedStorage->Initialize())
	{
		UE_LOG(LogUSDPregenInterchange, Error, TEXT("Failed to initialize storage plugin: %s"), *StorageOptions.StoragePluginName);

		NotifyFailedImport();

		return;
	}

	// Import via Interchange
	UUSDPregenContext* PregenContext = NewObject<UUSDPregenContext>();
	PregenContext->SetUsdStage(EntryStage);
	PregenContext->SceneDiscovery = MakeShared<UE::UsdPregen::FSceneDiscovery>(SceneDiscovery);
	PregenContext->SceneDiscoveryResults = SceneDiscoveryResults;
	PregenContext->Storage = ScopedStorage->Storage;
	PregenContext->AllowedTargetUid = ImportOptions.TargetUid;

	UInterchangeSourceData* SourceData = NewObject<UInterchangeSourceData>();
	SourceData->SetFilename(FilePath);
	SourceData->SetContextObjectByTag(UE::Interchange::USD::USDContextTag, PregenContext);

	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	if (!InterchangeManager.CanTranslateSourceData(SourceData))
	{
		UE_LOG(LogUSDPregenInterchange, Error, TEXT("Interchange cannot translate source data: %s"), *FilePath);

		NotifyFailedImport();

		return;
	}

	// Save and override CVars
	static IConsoleVariable* InstancingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.InstancingAwareTranslation"));
	bool bPreviousInstancingValue = true;
	if (InstancingCVar)
	{
		bPreviousInstancingValue = InstancingCVar->GetBool();
		InstancingCVar->Set(false);
	}

	// When the import targets a specific prim subtree, narrow the USD translator's PrimsToImport
	// for the duration of this import. We mutate the CDO because the translator's GetSettings()
	// duplicates from it lazily on first read, and there's no per-import override channel
	// otherwise. The previous value is restored in OnSceneImportDoneNative below.
	UInterchangeUsdTranslatorSettings* TranslatorSettingsCDO = GetMutableDefault<UInterchangeUsdTranslatorSettings>();
	const FString& OptionsInitialPath = ImportOptions.DiscoveryOptions.InitialPath;
	const bool bOverrodePrimsToImport = !OptionsInitialPath.IsEmpty() && TranslatorSettingsCDO != nullptr;
	TArray<FString> PreviousPrimsToImport;
	if (bOverrodePrimsToImport)
	{
		PreviousPrimsToImport = TranslatorSettingsCDO->PrimsToImport;
		TranslatorSettingsCDO->PrimsToImport = {OptionsInitialPath};
	}

	// Shared state for the callbacks (must outlive this function scope).
	// We accumulate products per target in-memory during the per-asset callback,
	// then save all manifests in the completion callback. This is necessary because
	// the JsonStoragePlugin is write-once and won't overwrite existing manifests,
	// so we can't do incremental saves when a target produces multiple assets.
	TSharedRef<UE::UsdPregen::FSceneDiscovery::ResultMap> SharedSceneDiscoveryResults = MakeShared<UE::UsdPregen::FSceneDiscovery::ResultMap>(MoveTemp(SceneDiscoveryResults));
	// Shared traversal for TargetData lookups in the per-asset callback. The
	// PregenContext also holds a copy, but capturing here keeps the lambda
	// independent of the context's UObject lifetime.
	TSharedRef<UE::UsdPregen::FSceneDiscovery> SharedSceneDiscovery = PregenContext->SceneDiscovery.ToSharedRef();
	UE::FUsdStage SharedEntryStage = EntryStage;

	// Map from target UID string -> accumulated manifest with all products for that target
	TSharedRef<TMap<FString, UE::UsdPregen::FManifest>> AccumulatedManifests = MakeShared<TMap<FString, UE::UsdPregen::FManifest>>();

	FImportAssetParameters ImportAssetParameters;
	ImportAssetParameters.bIsAutomated = ImportOptions.bAutomated;
	ImportAssetParameters.OverridePipelines = GetDefault<UUSDPregenSettings>()->Pipelines;

	// Final on-disk package files. Note this is only populated when ImportOptions.bAutoSavePackages is true.
	TSharedRef<TArray<FString>> SavedPackageFilePaths = MakeShared<TArray<FString>>();
	TSharedRef<bool> bSuccess = MakeShared<bool>(true);

	const FString AllowedTargetUid = ImportOptions.TargetUid;

	// TODO: This would also be done from the pipeline's PostFactoryPipeline if needed
	ImportAssetParameters.OnAssetDoneNative.BindLambda(
		[SharedSceneDiscoveryResults, SharedSceneDiscovery, SharedEntryStage, AccumulatedManifests, bSuccess, AllowedTargetUid](UObject* Asset)
		{
			// TODO: if bSuccess is false here it means we failed to save one or more packages when
			// bSavePackages was set to true (i.e. headless/worker mode)
			// 
			// We probably want to:
			// 1. Clean up all the packages as best we can.
			// 2. Skip the manifest save. 
			
			// Skip some assets for which we expect to not have a prim path anyway
			if (!Asset || Asset->IsA<UInterchangeSceneImportAsset>() || Asset->IsA<ULevelSequence>())
			{
				return;
			}

			FString PrimPath = GetPrimPathFromAsset(Asset);
			if (PrimPath.IsEmpty())
			{
				UE_LOG(
					LogUSDPregenInterchange,
					Warning,
					TEXT("Pregen: Could not determine prim path for imported asset '%s'"),
					*Asset->GetPathName()
				);
				return;
			}

			// Walk up the prim path hierarchy to find the owning pregen target
			const TArray<UE::UsdPregen::FTargetUid>* TargetUids = nullptr;
			for (UE::FSdfPath SearchPrimPath{*PrimPath}; !SearchPrimPath.IsEmpty(); SearchPrimPath = SearchPrimPath.GetParentPath())
			{
				TargetUids = SharedSceneDiscoveryResults->Find(SearchPrimPath);
				if (TargetUids && !TargetUids->IsEmpty())
				{
					break;
				}
				TargetUids = nullptr;
			}

			if (!TargetUids)
			{
				// This asset's prim is not under any pregen target
				return;
			}

			const UE::UsdPregen::FTargetUid& TargetUid = (*TargetUids)[0];
			const FString TargetUidStr = TargetUid.GetString();

			// Skip targets that don't match the active allow-list (worker imports).
			// The pipeline already disables the corresponding factory nodes, but assets may
			// arrive here from prior pipelines or from translated subtrees that didn't match.
			if (!AllowedTargetUid.IsEmpty() && TargetUidStr != AllowedTargetUid)
			{
				return;
			}

			// Get the node UID from the asset's import data
			FString NodeUniqueId;
			if (UInterchangeAssetImportData* ImportData = UInterchangeAssetImportData::GetFromObject(Asset))
			{
				NodeUniqueId = ImportData->NodeUniqueID;
			}

			// Get the USD prim type from the entry stage
			FString UsdPrimType;
			if (SharedEntryStage)
			{
				UE::FUsdPrim Prim = SharedEntryStage.GetPrimAtPath(UE::FSdfPath(*PrimPath));
				if (Prim)
				{
					UsdPrimType = Prim.GetTypeName().ToString();
				}
			}

			// Find or create the in-memory manifest for this target. The
			// manifest's target uid is sourced from its attached TargetData,
			// so we must attach one before doing anything else.
			UE::UsdPregen::FManifest& Manifest = AccumulatedManifests->FindOrAdd(TargetUidStr);
			if (!Manifest.GetTargetData())
			{
				UE::UsdPregen::FTargetData TargetData
					= SharedSceneDiscovery->GetTargetData(TargetUid);
				if (!TargetData)
				{
					AccumulatedManifests->Remove(TargetUidStr);
					UE_LOG(
						LogUSDPregenInterchange,
						Warning,
						TEXT("Pregen: No target data available from traversal for target '%s' - skipping product '%s'"),
						*TargetUidStr,
						*Asset->GetPathName()
					);
					return;
				}

				Manifest.SetTargetData(TargetData);
			}

			UE::UsdPregen::FProduct NewProduct;
			NewProduct.UPackagePath = Asset->GetPathName();
			NewProduct.UClass = Asset->GetClass()->GetPathName();
			NewProduct.UNodeId = NodeUniqueId;
			NewProduct.UsdPrimPath = PrimPath;
			NewProduct.UsdPrimType = UsdPrimType;
			Manifest.AddProduct(NewProduct);

			UE_LOG(
				LogUSDPregenInterchange,
				Verbose,
				TEXT("Pregen: Accumulated product '%s' for target '%s' (prim: %s)"),
				*Asset->GetPathName(),
				*TargetUidStr,
				*PrimPath
			);
		}
	);

	// Save packages, if requested.
	const bool bAutoSavePackages = ImportOptions.bAutoSavePackages;
	const bool bAutomated = ImportOptions.bAutomated;
	ImportAssetParameters.OnAssetsImportDoneNative.BindLambda([bAutoSavePackages, bAutomated, SavedPackageFilePaths, bSuccess, FilePath]
	(const TArray<UObject*>& ImportedAssets)
		{
			if (!bAutoSavePackages || !bAutomated)
			{
				return;
			}

			// Scope saving to the configured import content directory so we never touch random
			// unsaved assets that happen to be dirty in the worker's project. If the setting is
			// empty we cannot safely scope, so skip saving entirely rather than saving everything.
			FString ImportContentPath = GetDefault<UUSDPregenSettings>()->ImportContentPath.Path;
			if (ImportContentPath.IsEmpty())
			{
				*bSuccess = false;
				UE_LOG(LogUSDPregenInterchange, Warning, TEXT("Pregen ImportContentPath is empty; skipping auto-save of packages for %s"), *FilePath);
				return;
			}
			if (!ImportContentPath.EndsWith(TEXT("/")))
			{
				ImportContentPath += TEXT("/");
			}

			// Collect dirty packages under the import content path.
			//
			// Ideally we'd use the provided list of ImportedAssets, but we run into trouble with the
			// VT/nonVT textures that are produced on-demand by InterchangeMaterialFactory's
			// GetVirtualTextureStreamingMatchedTexture() function. Those UObjects are additional to the
			// main UObject the factory produces, and don't get properly registered on the AsyncHelper
			// like the main object does from FTaskImportObject_GameThread. This means those VT/nonVT
			// textures will be missing from the ImportedAssets list, and we'd otherwise miss saving them.
			//
			// In the future we can hopefully register these ancillary objects properly so that they
			// naturally show up on the ImportedAssets list.
			TArray<UPackage*> DirtyPackages;
			FEditorFileUtils::GetDirtyPackages(
				DirtyPackages,
				[&ImportContentPath](UPackage* Package) -> bool
				{
					// Return true to IGNORE the package (not inside the pregen content folder path)
					return !Package || !Package->GetName().StartsWith(ImportContentPath);
				}
			);

			TArray<UObject*> DirtyAssets;
			DirtyAssets.Reserve(DirtyPackages.Num());
			for (UPackage* Package : DirtyPackages)
			{
				if (UObject* Asset = Package->FindAssetInPackage(RF_Public | RF_Standalone))
				{
					DirtyAssets.Add(Asset);
				}
			}

			FAssetCompilingManager::Get().FinishCompilationForObjects(DirtyAssets);

			for (UObject* Asset : DirtyAssets)
			{
				if (UTexture* Texture = Cast<UTexture>(Asset))
				{
					Texture->UpdateResource();
				}
				ThumbnailTools::GenerateThumbnailForObjectToSaveToDisk(Asset);
			}

			const bool bOnlyDirty = true;
			if (!UEditorLoadingAndSavingUtils::SavePackages(DirtyPackages, bOnlyDirty))
			{
				*bSuccess = false;
				UE_LOG(LogUSDPregenInterchange, Error, TEXT("Failed to save one or more packages while processing file %s"), *FilePath);
				return;
			}

			TArray<FString> PackageFilePaths;
			PackageFilePaths.Reserve(DirtyPackages.Num());
			for (UPackage* Package : DirtyPackages)
			{
				PackageFilePaths.Add(FPackageName::LongPackageNameToFilename(Package->GetName(), FPackageName::GetAssetPackageExtension()));
			}

			*SavedPackageFilePaths = PackageFilePaths;
		});

	// Completion callback: store all accumulated manifests, optionally persist them,
	// restore CVars + translator settings, and fire OnImportDone.
	//
	// TODO: This is the only reason why we can't have most of this code inside the USDPregenPipeline: We need a single
	// callback executed exactly once after the import is complete, that we can use to save our manifests.
	//
	// TODO: This can possibly be combined with OnAssetsImportDoneNative above?
	ImportAssetParameters.OnSceneImportDoneNative.BindLambda(
		[
			bPreviousInstancingValue,
			AccumulatedManifests,
			ScopedStorage,
			SavedPackageFilePaths,
			bSuccess,
			bAutoSavePackages,
			bOverrodePrimsToImport,
			PreviousPrimsToImport = MoveTemp(PreviousPrimsToImport),
			ImportOptions,
			OnImportDone = MoveTemp(OnImportDone)
		](const TArray<UObject*>&)
		{
			// Store all accumulated manifests, then (optionally) persist them.
			TArray<UE::UsdPregen::FTargetUid> StoredTargets;
			for (TPair<FString, UE::UsdPregen::FManifest>& Pair : *AccumulatedManifests)
			{
				const FString& TargetUidStr = Pair.Key;
				UE::UsdPregen::FManifest& Manifest = Pair.Value;

				if (!Manifest.IsValid())
				{
					UE_LOG(
						LogUSDPregenInterchange,
						Warning,
						TEXT("Pregen: Skipping invalid manifest for target '%s'"),
						*TargetUidStr
					);
					continue;
				}

				UE::UsdPregen::FTargetUid TargetUid = Manifest.GetTargetUid();

				UE::UsdPregen::FManifestPayload Payload = ScopedStorage->Storage.SerializeManifest(Manifest);
				UE::UsdPregen::FManifestSaveResult StoreResult = ScopedStorage->Storage.StoreManifestPayload(TargetUid, Payload);

				if (StoreResult.Status == UE::UsdPregen::EManifestSaveStatus::Saved)
				{
					UE_LOG(
						LogUSDPregenInterchange,
						Verbose,
						TEXT("Pregen: Stored manifest for target '%s' with %d product(s)"),
						*TargetUidStr,
						Manifest.GetProducts().Num()
					);
					StoredTargets.Add(TargetUid);
				}
				else if (StoreResult.Status == UE::UsdPregen::EManifestSaveStatus::NotSaved)
				{
					UE_LOG(
						LogUSDPregenInterchange,
						Verbose,
						TEXT("Pregen: Manifest for target '%s' already exists, skipping: %s"),
						*TargetUidStr,
						*StoreResult.Message
					);
				}
				else
				{
					UE_LOG(
						LogUSDPregenInterchange,
						Warning,
						TEXT("Pregen: Failed to store manifest for target '%s': %s"),
						*TargetUidStr,
						*StoreResult.Message
					);
					*bSuccess = false;
				}
			}

			// In auto-save mode, also persist the manifests to their final destination.
			// For filesystem-backed plugins this is a no-op (Store already wrote to disk);
			// for UAsset-backed plugins this is what actually flushes the package out.
			if (bAutoSavePackages)
			{
				for (const UE::UsdPregen::FTargetUid& TargetUid : StoredTargets)
				{
					UE::UsdPregen::FManifestSaveResult PersistResult = ScopedStorage->Storage.PersistManifestPayload(TargetUid);
					if (PersistResult.Status == UE::UsdPregen::EManifestSaveStatus::Error)
					{
						UE_LOG(
							LogUSDPregenInterchange,
							Warning,
							TEXT("Pregen: Failed to persist manifest for target '%s': %s"),
							*TargetUid.GetString(),
							*PersistResult.Message
						);
						*bSuccess = false;
					}
				}
			}

			if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.InstancingAwareTranslation")))
			{
				CVar->Set(bPreviousInstancingValue);
			}

			if (bOverrodePrimsToImport)
			{
				if (UInterchangeUsdTranslatorSettings* RestoreSettingsCDO = GetMutableDefault<UInterchangeUsdTranslatorSettings>())
				{
					RestoreSettingsCDO->PrimsToImport = PreviousPrimsToImport;
				}
			}

			UE_LOG(LogUSDPregenInterchange, Log, TEXT("Pregen import completed (success=%s, saved=%d)"),
				*bSuccess ? TEXT("true") : TEXT("false"),
				SavedPackageFilePaths->Num()
			);

			if (OnImportDone)
			{
				OnImportDone(ImportOptions, *bSuccess, *SavedPackageFilePaths);
			}
		}
	);

	const FString& ContentPath = GetDefault<UUSDPregenSettings>()->ImportContentPath.Path;

	InterchangeManager.ImportSceneAsync(ContentPath, SourceData, ImportAssetParameters);
}

IMPLEMENT_MODULE(FUSDPregenInterchangeModule, USDPregenInterchange)
