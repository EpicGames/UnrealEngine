// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenPipeline.h"

#include "USDPregenContext.h"
#include "UsdPregenWrappers/AssetDefinitionRegistry.h"
#include "UsdPregenWrappers/ExtAssetDefinition.h"
#include "UsdPregenWrappers/Manifest.h"
#include "UsdPregenWrappers/ManifestTypes.h"
#include "UsdPregenWrappers/Target.h"

#include "SchemaHandlers/SchemaHandlerUtils.h"
#include "Usd/InterchangeUsdDefinitions.h"

#include "InterchangeActorFactoryNode.h"
#include "InterchangePhysicsAssetFactoryNode.h"
#include "InterchangeSceneImportAssetFactoryNode.h"
#include "InterchangeSkeletonFactoryNode.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

DEFINE_LOG_CATEGORY_STATIC(LogUSDPregenPipeline, Log, All);

namespace UE::USDPregenPipeline::Private
{
	FString ReadPrimPathFromNodeAttributes(const UInterchangeBaseNode* Node)
	{
		if (!Node)
		{
			return FString{};
		}

		TArray<FInterchangeUserDefinedAttributeInfo> AttributeInfos = UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttributeInfos(Node);
		for (const FInterchangeUserDefinedAttributeInfo& Info : AttributeInfos)
		{
			if (!Info.Name.Contains(UE::Interchange::USD::PrimPathAttributeKeyString))
			{
				continue;
			}

			FString PrimPath;
			TOptional<FString> PayloadKey;
			if (UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(Node, Info.Name, PrimPath, PayloadKey) && !PrimPath.IsEmpty())
			{
				return PrimPath;
			}
		}
		return FString{};
	}

	FString FindPrimPathForFactoryNode(
		const UInterchangeFactoryBaseNode* FactoryNode,
		const UInterchangeBaseNodeContainer& NodeContainer
	)
	{
		FString PrimPath = ReadPrimPathFromNodeAttributes(FactoryNode);
		if (!PrimPath.IsEmpty())
		{
			return PrimPath;
		}

		if (const UInterchangeSkeletonFactoryNode* SkeletonFactory = Cast<const UInterchangeSkeletonFactoryNode>(FactoryNode))
		{
			FString RootJointUid;
			if (SkeletonFactory->GetCustomRootJointUid(RootJointUid))
			{
				for (FString CurrentUid = RootJointUid; !CurrentUid.IsEmpty();)
				{
					const UInterchangeBaseNode* CurrentNode = NodeContainer.GetNode(CurrentUid);
					if (!CurrentNode)
					{
						break;
					}

					PrimPath = ReadPrimPathFromNodeAttributes(CurrentNode);
					if (!PrimPath.IsEmpty())
					{
						return PrimPath;
					}

					CurrentUid = CurrentNode->GetParentUid();
				}
			}
		}
		else if (const UInterchangePhysicsAssetFactoryNode* PhysicsAssetFactory = Cast<const UInterchangePhysicsAssetFactoryNode>(FactoryNode))
		{
			FString SkeletalMeshUid;
			if (PhysicsAssetFactory->GetCustomSkeletalMeshUid(SkeletalMeshUid))
			{
				if (const UInterchangeFactoryBaseNode* SkeletalMeshFactory = NodeContainer.GetFactoryNode(SkeletalMeshUid))
				{
					PrimPath = ReadPrimPathFromNodeAttributes(SkeletalMeshFactory);
					if (!PrimPath.IsEmpty())
					{
						return PrimPath;
					}
				}
			}
		}

		return FString{};
	}

	// Finds the ideal OutSubPath for a factory node according to pregen, also returning whether
	// we should disable this node or use a custom reference object path instead (for e.g. when
	// reusing existing assets). The resolved USD prim path is returned via OutPrimPath so the
	// caller can stamp it back onto the factory node (needed for Skeleton/PhysicsAsset, which
	// don't otherwise carry the attribute -- see the SetPrimPath call in ExecutePipeline).
	void ResolveTargetSubPath(
		const UInterchangeFactoryBaseNode* FactoryNode,
		const FString& FactoryNodeUid,
		const UInterchangeBaseNodeContainer& NodeContainer,
		const UUSDPregenContext& PregenCtx,
		const UE::UsdPregen::FAssetDefinitionRegistry& Registry,
		FString& OutSubPath,
		bool& bOutShouldDisableNode,
		FSoftObjectPath& OutCustomReferenceObject,
		FString& OutPrimPath
	)
	{
		OutSubPath.Reset();
		bOutShouldDisableNode = false;
		OutCustomReferenceObject.Reset();
		OutPrimPath.Reset();

		const FString PrimPath = FindPrimPathForFactoryNode(FactoryNode, NodeContainer);
		if (PrimPath.IsEmpty())
		{
			return;
		}
		OutPrimPath = PrimPath;

		// Walk up the prim path hierarchy to find the owning pregen target.
		// The prim path on the factory node is typically a leaf geometry prim (e.g.
		// /Root/MeasuringSpoon_3/Geom/MeasuringSpoon) while the target is an ancestor
		// (e.g. /Root/MeasuringSpoon_3).
		const TArray<UE::UsdPregen::FTargetUid>* TargetUids = nullptr;
		for (UE::FSdfPath SearchPrimPath{*PrimPath}; !SearchPrimPath.IsEmpty(); SearchPrimPath = SearchPrimPath.GetParentPath())
		{
			TargetUids = PregenCtx.SceneDiscoveryResults.Find(SearchPrimPath);
			if (TargetUids && !TargetUids->IsEmpty())
			{
				break;
			}
			TargetUids = nullptr;
		}
		if (!TargetUids)
		{
			return;
		}
		const UE::UsdPregen::FTargetUid& TargetUid = (*TargetUids)[0];

		// Factory nodes outside the allow-listed target need to be disabled so the import
		// doesn't produce assets we don't care about (worker imports).
		if (!PregenCtx.AllowedTargetUid.IsEmpty() && TargetUid.GetString() != PregenCtx.AllowedTargetUid)
		{
			bOutShouldDisableNode = true;
			return;
		}

		// Get target data from our target
		UE::UsdPregen::FTargetData TargetData = PregenCtx.SceneDiscovery->GetTargetData(TargetUid);
		if (!TargetData.IsValid())
		{
			return;
		}

		// Build definitions chain from our target data
		TArray<UE::UsdPregen::FExtAssetDefinition> Definitions;
		TArray<UE::UsdPregen::FTargetDefinitionEntry> Infos = TargetData.GetDefinitionEntries();
		Definitions.Reserve(Infos.Num());
		for (const UE::UsdPregen::FTargetDefinitionEntry& Info : Infos)
		{
			UE::UsdPregen::FExtAssetDefinition Defn = Registry.GetDefinition(Info.GetDefinition().GetUniqueId());
			if (Defn)
			{
				Definitions.Add(MoveTemp(Defn));
			}
		}
		if (Definitions.IsEmpty())
		{
			return;
		}

		// Check if this target has a manifest with existing products we can reuse
		UE::UsdPregen::FManifestLoadResult LoadResult = PregenCtx.Storage.LoadManifestPayload(TargetUid);
		if (LoadResult.Status == UE::UsdPregen::EManifestLoadStatus::Loaded)
		{
			UE::UsdPregen::FManifest Manifest = PregenCtx.Storage.DeserializeManifestPayload(LoadResult.Payload);
			if (Manifest.IsValid())
			{
				// Match by factory node UID -- the manifest stores the UID of the factory
				// node that originally produced each product, and factory UIDs are
				// deterministic (built from prim paths), so they match across imports
				for (const UE::UsdPregen::FProduct& Product : Manifest.GetProducts())
				{
					if (Product.UNodeId != FactoryNodeUid || Product.UPackagePath.IsEmpty())
					{
						continue;
					}

					FSoftObjectPath SoftPath{Product.UPackagePath};
					UObject* ExistingAsset = SoftPath.TryLoad();
					if (ExistingAsset)
					{
						UE_LOGF(LogUSDPregenPipeline, Verbose, "Reusing existing asset '%ls' for factory node '%ls'", *Product.UPackagePath, *FactoryNodeUid);
						OutCustomReferenceObject = SoftPath;
						bOutShouldDisableNode = true;
						break;
					}
				}
			}
		}

		// Compute where to place the asset in the content browser
		const FString AssetType = FactoryNode->GetObjectClass() ? FactoryNode->GetObjectClass()->GetName() : FString{};
		OutSubPath = PregenCtx.Storage.GetPackageSubPathForUAsset(TargetUid, Definitions, AssetType);
		if (!OutSubPath.IsEmpty())
		{
			UE_LOGF(LogUSDPregenPipeline, Verbose, "Using SubPath '%ls' for factory node '%ls'", *OutSubPath, *FactoryNodeUid);
		}
	}
}

void UUSDPregenPipeline::ExecutePipeline(
	UInterchangeBaseNodeContainer* BaseNodeContainer,
	const TArray<UInterchangeSourceData*>& SourceDatas,
	const FString& ContentBasePath
)
{
#if USE_USD_SDK
	if (!BaseNodeContainer || SourceDatas.IsEmpty())
	{
		return;
	}

	// Retrieve the pregen context by casting the USD context (UUSDPregenContext derives from UInterchangeUsdContext)
	UUSDPregenContext* PregenCtx = Cast<UUSDPregenContext>(SourceDatas[0]->GetContextObjectByTag(UE::Interchange::USD::USDContextTag));
	if (!PregenCtx || !PregenCtx->SceneDiscovery)
	{
		return;
	}

	UE::UsdPregen::FAssetDefinitionRegistry Registry = UE::UsdPregen::FAssetDefinitionRegistry::GetInstance();

	// Cache the subpath to use for scene and uncategorized assets here both so that we compute it only
	// once, but also because since it involves a timestamp, if it takes more than 1 second to loop over all
	// the nodes we may end up with different SubPaths for some. Asset factory nodes that fail to resolve
	// a per-target SubPath (e.g. no prim path, no matching target, empty storage path) fall back to this
	// scenes/ folder so they still land somewhere predictable instead of dropping out of the content browser.
	FString SceneSubPath;

	const bool bIsWorker = !PregenCtx->AllowedTargetUid.IsEmpty();

	BaseNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>(
		[BaseNodeContainer, &SceneSubPath, PregenCtx, &Registry, &SourceDatas, bIsWorker](const FString& FactoryNodeUid, UInterchangeFactoryBaseNode* FactoryNode)
		{
			// We only care about produced assets for now
			if (FactoryNode->IsA<UInterchangeActorFactoryNode>())
			{
				return;
			}

			// Worker imports never want the SceneImportAsset: it's used to reimport the whole scene,
			// but a worker is only importing one target at a time. Letting it through would put each
			// worker's SceneImportAsset into the /scenes/ fallback folder, polluting the shared
			// PregenContent plugin with one /scenes/ subfolder per worker task.
			if (bIsWorker && FactoryNode->IsA<UInterchangeSceneImportAssetFactoryNode>())
			{
				FactoryNode->SetEnabled(false);
				return;
			}

			FString SubPath;
			bool bShouldDisableNode = false;
			FSoftObjectPath CustomReferenceObject;
			FString ResolvedPrimPath;
			UE::USDPregenPipeline::Private::ResolveTargetSubPath(
				FactoryNode,
				FactoryNodeUid,
				*BaseNodeContainer,
				*PregenCtx,
				Registry,
				SubPath,
				bShouldDisableNode,
				CustomReferenceObject,
				ResolvedPrimPath
			);

			// Make sure our factory node has whatever prim path we used.
			// This is important because we may have jumped to other nodes to find the right prim path,
			// but we need to make sure it's in this exact factory node so that it can be retrieved from
			// the asset later			
			if (!ResolvedPrimPath.IsEmpty())
			{
				UE::Interchange::USD::SetPrimPath(*FactoryNode, ResolvedPrimPath);
			}			

			// We should always have a subpath, or else the asset will fall off completely back to the ImportContentPath folder,
			// and we never want that to happen
			if (SubPath.IsEmpty())
			{
				if (SceneSubPath.IsEmpty())
				{
					FString BaseName = FPaths::GetBaseFilename(SourceDatas[0]->GetFilename());
					FString Timestamp = FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S"));
					SceneSubPath = FString::Printf(TEXT("scenes/%s_%s"), *BaseName, *Timestamp);
				}
				SubPath = SceneSubPath;
				UE_LOGF(LogUSDPregenPipeline, Verbose, "Falling back to scenes folder for factory node '%ls': SubPath '%ls'", *FactoryNodeUid, *SubPath);
			}
			FactoryNode->SetCustomSubPath(SubPath);

			// Disable the factory node if we'be been told to do so (i.e. we're reusing an asset, or skipping as
			// it's not from our target)
			if (bShouldDisableNode)
			{
				FactoryNode->SetEnabled(false);
			}

			// Set the reference to an existing asset if we have one (i.e. we're reusing an asset)
			if (CustomReferenceObject.IsValid())
			{
				FactoryNode->SetCustomReferenceObject(CustomReferenceObject);
			}
		}
	);
#endif // USE_USD_SDK
}
