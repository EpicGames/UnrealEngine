// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDPregenSettings)

UUSDPregenSettings::UUSDPregenSettings()
{
	ImportContentPath.Path = TEXT("/Game");

	Pipelines.Add(FSoftObjectPath(TEXT("/USDPregen/Pipelines/USDPregenAssetsPipeline.USDPregenAssetsPipeline")));
	Pipelines.Add(FSoftObjectPath(TEXT("/USDPregen/Pipelines/USDPregenLevelPipeline.USDPregenLevelPipeline")));
	Pipelines.Add(FSoftObjectPath(TEXT("/USDPregen/Pipelines/USDPregenUSDPipeline.USDPregenUSDPipeline")));
	Pipelines.Add(FSoftObjectPath(TEXT("/Interchange/Pipelines/DefaultMaterialXPipeline.DefaultMaterialXPipeline")));
	Pipelines.Add(FSoftObjectPath(TEXT("/USDPregen/Pipelines/USDPregenPipeline.USDPregenPipeline")));
}

#if WITH_EDITOR
void UUSDPregenSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UDeveloperSettings::PostEditChangeProperty(PropertyChangedEvent);

	SaveConfig();
}
#endif	// WITH_EDITOR
