// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenInterchangeModule.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "USDPregenBlueprintLibrary.generated.h"

DECLARE_DYNAMIC_DELEGATE_ThreeParams(
	FUSDPregenOnImportDoneDynamic,
	const FPregenImportOptions&, ImportOptions,
	bool, bSuccess,
	const TArray<FString>&, SavedPackageFilePaths);

UCLASS()
class USDPREGENINTERCHANGE_API UUSDPregenBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Kicks off an asynchronous USD pregen import described by ImportOptions. If
	// bound, OnImportDone fires on the game thread once manifests have been
	// stored, optionally persisted, and (when ImportOptions.bAutoSavePackages is
	// true) imported asset packages have been saved to disk.
	//
	// This wraps FUSDPregenInterchangeModule::ImportFile for Python and blueprint usage.
	// OnImportDone is optional.
	//
	// === Python usage ===
	//   import unreal
	//
	//   import_options = unreal.PregenImportOptions()
	//   import_options.source_file_path = '/abs/path/to/scene.usda'
	//
	//   # Plugin choice and discovery/storage options all live on the
	//   # nested option structs. Leave any field empty for the default.
	//   import_options.discovery_options.discovery_plugin_name = 'my_studio_discovery'
	//   import_options.discovery_options.definition_prefix = 'Shot_42_'
	//   import_options.discovery_options.initial_path = '/Root/Hero'
	//   import_options.storage_options.storage_plugin_name = 'json_storage'
	//   import_options.storage_options.package_sub_path_template = 'assets/${DEFINITION_NAME}/${PERMUTATION_ID}'
	//
	//   # Without callback (fire-and-forget):
	//   unreal.USDPregenBlueprintLibrary.import_file(import_options, unreal.USDPregenOnImportDoneDynamic())
	//
	//   # With callback:
	//   def on_done(completed_options, success, saved_paths):
	//       print('Done!')
	//
	//   delegate = unreal.USDPregenOnImportDoneDynamic()
	//   delegate.bind_callable(on_done)
	//   unreal.USDPregenBlueprintLibrary.import_file(import_options, delegate)
	//
	UFUNCTION(BlueprintCallable, Category = "USD Pregen", meta = (AutoCreateRefTerm = "OnImportDone"))
	static void ImportFile(const FPregenImportOptions& ImportOptions, const FUSDPregenOnImportDoneDynamic& OnImportDone);
};
