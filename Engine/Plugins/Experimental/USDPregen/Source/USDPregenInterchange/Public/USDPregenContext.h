// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeUsdContext.h"
#include "UsdPregenWrappers/StoragePlugin.h"
#include "UsdPregenWrappers/SceneDiscovery.h"

#include "CoreMinimal.h"

#include "USDPregenContext.generated.h"

/**
 * Pregen-aware USD context that overrides asset node UID generation to produce stable,
 * target-remapped UIDs. This causes all instances of the same pregen target to converge
 * to the same node UID, achieving deduplication.
 *
 * Set as the USD context on SourceData (via the standard "USD" tag) in place of
 * the base UInterchangeUsdContext.
 */
UCLASS()
class USDPREGENINTERCHANGE_API UUSDPregenContext : public UInterchangeUsdContext
{
	GENERATED_BODY()

public:
	virtual FString MakeAssetNodeUid(const UE::FUsdPrim& Prim, const FString& Prefix = FString(), const FString& Suffix = FString()) const override;

	// SceneDiscovery instance, needed for GetTargetData() lookups.
	// Wrapped in TSharedPtr because FSceneDiscovery has no default constructor.
	TSharedPtr<UE::UsdPregen::FSceneDiscovery> SceneDiscovery;

	// Map of prim path -> target UIDs from the discovery/traversal phase
	UE::UsdPregen::FSceneDiscovery::ResultMap SceneDiscoveryResults;

	// Storage plugin used for GetPackageSubPathForUAsset
	UE::UsdPregen::FStoragePlugin Storage;

	// When non-empty, only prims whose owning pregen target's UID string matches
	// this value are processed by the pipeline / accumulated into manifests.
	// Every other factory node is disabled. Empty = process every discovered target
	// (default editor behavior).
	FString AllowedTargetUid;
};
