// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Modules/ModuleInterface.h"
#include "Templates/Function.h"
#include "USDPregenWrapper.h"

#include "USDPregenInterchangeModule.generated.h"

// Describes a single pregen import. Passed to FUSDPregenInterchangeModule::ImportFile and
// also exposed to Blueprint / Python via UUSDPregenBlueprintLibrary.
//
// Forward-compatible with worker-driven imports that need to filter to a single target,
// apply permutation overlays, override plugins, etc.
USTRUCT(BlueprintType)
struct FPregenImportOptions
{
	GENERATED_BODY()

	// Required: USD file to import.
	UPROPERTY(BlueprintReadWrite, Category = "USD Pregen")
	FString SourceFilePath;

	// Optional filter to import only a specific target.
	// Empty means to import all discovered targets.
	UPROPERTY(BlueprintReadWrite, Category = "USD Pregen")
	FString TargetUid;

	// Optional USD layer to sublayer onto the entry stage to apply
	// permutation operations. Empty means no permutation layer.
	UPROPERTY(BlueprintReadWrite, Category = "USD Pregen")
	FString PermutationLayerPath;

	// Optional human-readable label for logs.
	UPROPERTY(BlueprintReadWrite, Category = "USD Pregen")
	FString Title;

	// Options for the discovery plugin, including the name of the
	// plugin to use. Empty values fall back to defaults
	UPROPERTY(BlueprintReadWrite, Category = "USD Pregen")
	FPregenDiscoveryOptions DiscoveryOptions;

	// Options for the storage plugin, including the name of the
	// plugin to use. Empty values fall back to defaults
	UPROPERTY(BlueprintReadWrite, Category = "USD Pregen")
	FPregenStorageOptions StorageOptions;

	// When true, ImportFile runs the underlying Interchange import in
	// automated mode (FImportAssetParameters::bIsAutomated = true), which
	// suppresses the modal pipeline-options dialog and any other UI prompts.
	UPROPERTY(BlueprintReadWrite, Category = "USD Pregen")
	bool bAutomated = true;

	// When true, ImportFile calls Storage->PersistManifestPayload(...)
	// for each target whose manifest was Stored, AND saves imported
	// asset packages to disk after import completes
	UPROPERTY(BlueprintReadWrite, Category = "USD Pregen")
	bool bAutoSavePackages = false;
};

class USDPREGENINTERCHANGE_API FUSDPregenInterchangeModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// Callback fired on the game thread when an async pregen import completes.
	// Receives the originating ImportOptions so callers can correlate to their submission,
	// a success bool, and the on-disk file paths for any UAssets that were saved
	// (only populated when ImportOptions.bAutoSavePackages was true).
	using FOnImportDone = TFunction<void(
		const FPregenImportOptions& ImportOptions,
		bool bSuccess,
		const TArray<FString>& SavedPackageFilePaths)>;

	// Runs a full pregen import described by ImportOptions via Interchange.
	// Optionally fires OnImportDone when the async import completes.
	static void ImportFile(const FPregenImportOptions& ImportOptions, FOnImportDone OnImportDone = nullptr);
};
