// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangePipelineBase.h"

#include "CoreMinimal.h"

#include "USDPregenPipeline.generated.h"

/**
 * Pipeline that handles managing the produced assets for USD Pregen
 */
UCLASS(BlueprintType)
class USDPREGENINTERCHANGE_API UUSDPregenPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

protected:
	virtual void ExecutePipeline(
		UInterchangeBaseNodeContainer* BaseNodeContainer,
		const TArray<UInterchangeSourceData*>& SourceDatas,
		const FString& ContentBasePath
	) override;
};
