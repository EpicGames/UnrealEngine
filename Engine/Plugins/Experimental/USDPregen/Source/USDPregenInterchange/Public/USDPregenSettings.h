// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/EngineTypes.h"

#include "USDPregenSettings.generated.h"

UCLASS(BlueprintType, config = USDPregen, defaultconfig, meta = (DisplayName = "USD Pregen"))
class USDPREGENINTERCHANGE_API UUSDPregenSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UUSDPregenSettings();

	/** Content directory where USD Pregen imports will place assets. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, config, Category = "USD Pregen", meta = (ContentDir))
	FDirectoryPath ImportContentPath;

	/**
	 * Template used to build the package sub-path for UAssets produced by USD Pregen imports.
	 * Recognized ${PLACEHOLDER} substitutions:
	 *   ${DEFINITION_NAME}    - leaf asset definition name (e.g. "tree_02")
	 *   ${DEFINITION_VERSION} - leaf asset definition version (e.g. "v2.1")
	 *   ${DEFINITION_UID}     - leaf asset definition unique id
	 *   ${PERMUTATION_ID}     - target's permutation id
	 *   ${ASSET_TYPE}         - Unreal asset type (e.g. "StaticMesh")
	 *   ${METADATA:KEY}       - looks up KEY in the leaf definition's metadata (populated from
	 *                           USD assetInfo, less the built-in keys). Nested dicts are
	 *                           descended via colon-separated paths, e.g.
	 *                           ${METADATA:nested:subcategory}. Missing keys, dict-valued
	 *                           leaves, and array-valued leaves collapse to the "_" sentinel.
	 *
	 * Values are sanitized to package-name-safe characters (alphanumeric and underscore).
	 * Empty values become the sentinel "_" so adjacent labels don't end up as orphans:
	 * e.g. "versions/${DEFINITION_VERSION}/" with no version becomes "versions/_/"
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, config, Category = "USD Pregen")
	FString PackageSubPathTemplate = TEXT("assets/${DEFINITION_NAME}/versions/${DEFINITION_VERSION}/permutations/${PERMUTATION_ID}");

	/**
	 * Pipeline stack used for USD Pregen scene imports via Interchange.
	 * These pipelines override the default Interchange pipeline stack when importing through the Pregen button.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, config, Category = "USD Pregen", meta = (AllowedClasses = "/Script/InterchangeCore.InterchangePipelineBase, /Script/InterchangeEngine.InterchangeBlueprintPipelineBase, /Script/InterchangeEngine.InterchangePythonPipelineAsset"))
	TArray<FSoftObjectPath> Pipelines;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
