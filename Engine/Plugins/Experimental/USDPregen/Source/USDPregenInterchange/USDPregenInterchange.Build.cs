// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class USDPregenInterchange : ModuleRules
	{
		public USDPregenInterchange(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseRTTI = false;
			bLegalToDistributeObjectCode = true;
			bUseUnity = false;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"DeveloperSettings",
					"InterchangeCore",
					"USDPregenWrapper"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"AssetTools",
					"ContentBrowser",
					"InterchangeEngine",
					"InterchangeFactoryNodes",
					"InterchangeOpenUSDImport",
					"LevelSequence",
					"UnrealEd",
					"UnrealUSDWrapper",
					"USDUtilities",
				}
			);
		}
	}
}
