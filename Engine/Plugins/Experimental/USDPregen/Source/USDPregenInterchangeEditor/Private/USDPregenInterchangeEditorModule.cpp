// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenInterchangeEditorModule.h"

#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "UnrealUSDWrapper.h"
#include "USDPregenInterchangeModule.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "Engine/Engine.h"

#define LOCTEXT_NAMESPACE "USDPregenInterchangeEditor"

namespace UE::USDPregenInterchangeEditor::Private
{
	bool OpenUsdFileDialog(FString& OutSelectedFile)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (!DesktopPlatform)
		{
			return false;
		}

		static FString LastDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT);
		TArray<FString> OpenedFiles;

		TArray<FString> TextExtensions;
		TArray<FString> BinaryExtensions;
		UnrealUSDWrapper::GetNativeFileFormats(TextExtensions, BinaryExtensions);

		TArray<FString> AllExtensions;
		AllExtensions.Append(TextExtensions);
		AllExtensions.Append(BinaryExtensions);

		FString JoinedExtensions = FString::Join(AllExtensions, TEXT(";*."));
		FString FileTypes = FString::Printf(TEXT("USD Files (*.%s)|*.%s"), *JoinedExtensions, *JoinedExtensions);

		bool bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			TEXT("Select USD file for Pregen Import"),
			LastDirectory,
			TEXT(""),
			FileTypes,
			EFileDialogFlags::None,
			OpenedFiles
		);

		if (!bOpened || OpenedFiles.IsEmpty())
		{
			return false;
		}

		// IDesktopPlatform::OpenFileDialog on Windows produces a path relative to
		// FPlatformProcess::BaseDir() when the selected file is on the same drive		
		OutSelectedFile = FPaths::ConvertRelativePathToFull(OpenedFiles[0]);
		LastDirectory = FPaths::GetPath(OutSelectedFile);
		FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_IMPORT, LastDirectory);
		return true;
	}

	void ShowPregenImportFailedToast(const FString& SourceFilePath)
	{
		TSharedRef<TWeakPtr<SNotificationItem>> WeakNotificationHolder = MakeShared<TWeakPtr<SNotificationItem>>();

		FNotificationInfo Info(LOCTEXT("PregenImportFailed", "Pregen Import Failure"));
		Info.SubText = FText::Format(
			LOCTEXT("PregenImportFailedSubText", "There were issues when importing '{0}'"),
			FText::FromString(SourceFilePath)
		);
		Info.bFireAndForget = false;
		Info.HyperlinkText = LOCTEXT("PregenImportFailedShowOutputLog", "Show Output Log");
		Info.Hyperlink = FSimpleDelegate::CreateLambda(
			[]()
			{
				FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog"));
			}
		);
		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("PregenImportFailedDismiss", "Dismiss"),
			LOCTEXT("PregenImportFailedDismissTooltip", "Dismiss this notification"),
			FSimpleDelegate::CreateLambda(
				[WeakNotificationHolder]()
				{
					if (TSharedPtr<SNotificationItem> Item = WeakNotificationHolder->Pin())
					{
						Item->SetCompletionState(SNotificationItem::CS_None);
						Item->ExpireAndFadeout();
					}
				}
			),
			SNotificationItem::CS_Fail
		));

		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			*WeakNotificationHolder = Notification;
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}

	void OnPregenImportClicked()
	{
		FString SelectedFile;
		if (!OpenUsdFileDialog(SelectedFile))
		{
			return;
		}

		FPregenImportOptions ImportOptions;
		ImportOptions.SourceFilePath = SelectedFile;
		ImportOptions.bAutomated = false;
		ImportOptions.bAutoSavePackages = false;

		FUSDPregenInterchangeModule::ImportFile(
			ImportOptions,
			[](const FPregenImportOptions& CompletedOptions, bool bSuccess, const TArray<FString>&)
			{
				if (!bSuccess)
				{
					ShowPregenImportFailedToast(CompletedOptions.SourceFilePath);
				}
			}
		);
	}
}

void FUSDPregenInterchangeEditorModule::StartupModule()
{
	auto RegisterItems = []()
	{
		if (!IsRunningCommandlet() && UToolMenus::IsToolMenuUIEnabled())
		{
			UToolMenu* ContentMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.AddQuickMenu");
			check(ContentMenu);

			FToolMenuSection& Section = ContentMenu->FindOrAddSection("ImportAssets");
			Section.InitSection("ImportAssets", LOCTEXT("ImportAssets_Label", "Import Assets"), FToolMenuInsert());
			Section.AddMenuEntry(
				TEXT("ImportPregen"),
				LOCTEXT("ImportPregenText", "USD Pregen Import"),
				LOCTEXT("ImportPregenTooltip", "Perform an import using the USD Pregen plugin via USD Interchange"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.USDStage"),
				FUIAction(FExecuteAction::CreateLambda(&UE::USDPregenInterchangeEditor::Private::OnPregenImportClicked))
			);
		}
	};

	if (GEngine)
	{
		RegisterItems();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterItems);
	}
}

void FUSDPregenInterchangeEditorModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FUSDPregenInterchangeEditorModule, USDPregenInterchangeEditor)
