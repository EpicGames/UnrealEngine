// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class USDPregenInterchangeEditor : ModuleRules
	{
		public USDPregenInterchangeEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseUnity = false;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"DesktopPlatform",
					"EditorFramework",
					"Engine",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"UnrealEd",
					"UnrealUSDWrapper",
					"USDPregenInterchange"
				}
			);
		}
	}
}
