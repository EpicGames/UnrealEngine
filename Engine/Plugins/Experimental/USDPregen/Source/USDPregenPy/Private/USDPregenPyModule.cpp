// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenPyModule.h"

#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

void FUSDPregenPyModule::StartupModule()
{

}

void FUSDPregenPyModule::ShutdownModule()
{

}

IMPLEMENT_MODULE(FUSDPregenPyModule, USDPregenPy)