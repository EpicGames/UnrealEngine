// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/assetDefinitionRegistry.h"
#include "UsdPregen/extAssetDefinition.h"

#include "USDIncludesStart.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/external/boost/python/operators.hpp"
#include "pxr/base/tf/pyResultConversions.h"
#include "pxr/base/tf/pySingleton.h"
#include "pxr/usd/usd/pyConversions.h"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

PREGEN_NAMESPACE_OPEN_SCOPE

struct _DefinitionRegistryTestHelper
{
	static void reset() {
		AssetDefinitionRegistry::GetInstance()._entries.clear();
	}
};

PREGEN_NAMESPACE_CLOSE_SCOPE

void wrapAssetDefinitionRegistry()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	using This = AssetDefinitionRegistry;
	using ThisPtr = pxr::TfWeakPtr<AssetDefinitionRegistry>;

	using RefPolicy = py::return_value_policy<py::reference_existing_object>;

	py::class_<This, ThisPtr, py::noncopyable>("AssetDefinitionRegistry", py::no_init)
		.def(pxr::TfPySingleton())
		.def("AddDefinition", &This::AddDefinition,
			RefPolicy(), "Returns a valid ExtAssetDefinition or None")
		.def("GetDefinition", &This::GetDefinition,
			RefPolicy(), "Returns a valid ExtAssetDefinition or None")
		.def("GetAllDefinitions", &This::GetAllDefinitions,
			py::return_value_policy<pxr::TfPySequenceToList>())
	;

	py::def("_reset_definition_registry", &_DefinitionRegistryTestHelper::reset);
}

#endif // USE_USD_SDK
