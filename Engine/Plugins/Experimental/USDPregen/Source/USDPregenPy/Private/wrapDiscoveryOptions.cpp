// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/discoveryOptions.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/pyEnum.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "USDIncludesEnd.h"

#include <memory>
#include <set>
#include <string>

PREGEN_NAMESPACE_USING_DIRECTIVE

namespace {

namespace py = PXR_BOOST_PYTHON_NAMESPACE;

void
_Options_setattr(py::object obj, py::str name, py::object value)
{
	// Helper to prevent invalid field names from going
	// undetected when set via python.
	static const std::set<std::string> fieldNames
	{
		"discoveryMode",
		"discoveryPluginName",
		"definitionPrefix",
		"initialPath",
		"purposes",
		"excludeVariantSets",
		"assetIdentifierFallback",
		"assetVersionFallback",
		"traversalPredicate"
	};

	const std::string nameStr = py::extract<std::string>(name);

	if (fieldNames.count(nameStr))
	{
		// Delegate to default behavior
		py::object object_type(py::handle<>(py::borrowed((PyObject*)&PyBaseObject_Type)));
		py::object base_setattr = object_type.attr("__setattr__");
		base_setattr(obj, name, value);
	}
	else
	{
		PyErr_SetString(PyExc_AttributeError,
			("Unknown attribute: " + nameStr).c_str());
		py::throw_error_already_set();
	}
}

} // anonymous namespace

void wrapDiscoveryOptions()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	pxr::TfPyWrapEnum<DiscoveryMode>();
	pxr::TfPyWrapEnum<IdentifierFallbackMode>();
	pxr::TfPyWrapEnum<VersionFallbackMode>();

	typedef DiscoveryOptions This;

	py::class_<This>("DiscoveryOptions")
		.def(py::init<>())

		.def_readwrite("discoveryMode",
			&This::discoveryMode)

		.def_readwrite("discoveryPluginName",
			&This::discoveryPluginName)

		.def_readwrite("definitionPrefix",
			&This::definitionPrefix)

		.def_readwrite("purposes",
			&This::purposes)

		.def_readwrite("excludeVariantSets",
			&This::excludeVariantSets)

		.def_readwrite("initialPath",
			&This::initialPath)

		.def_readwrite("assetIdentifierFallback",
			&This::assetIdentifierFallback)

		.def_readwrite("assetVersionFallback",
			&This::assetVersionFallback)

		.def_readwrite("traversalPredicate",
			&This::traversalPredicate)

		.def("__repr__", &This::DumpToString)

		.def("__setattr__", &_Options_setattr)
		.def(py::self == py::self)
		.def(py::self != py::self)
	;
}

#endif // USE_USD_SDK
