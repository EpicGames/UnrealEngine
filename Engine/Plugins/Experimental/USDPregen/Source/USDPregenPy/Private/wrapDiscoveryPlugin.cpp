// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/discoveryPlugin.h"

#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/discoveryOptions.h"
#include "UsdPregen/discoveryPluginRegistry.h"
#include "UsdPregen/primPermutation.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/pyLock.h"
#include "pxr/base/tf/pyPolymorphic.h"
#include "pxr/base/tf/pyResultConversions.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/external/boost/python/make_constructor.hpp"
#include "pxr/external/boost/python/stl_iterator.hpp"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/stage.h"
#include "USDIncludesEnd.h"

#include "wrapperHelpers.h"

#include <string>
#include <vector>

PREGEN_NAMESPACE_USING_DIRECTIVE

PXR_NAMESPACE_USING_DIRECTIVE

namespace
{

namespace py = PXR_BOOST_PYTHON_NAMESPACE;

class DiscoveryPluginWrapper
	: public DiscoveryPlugin
	, public pxr::TfPyPolymorphic<DiscoveryPlugin>
	, protected internal::py::VirtualDispatchHelper
{
public:

	using This = DiscoveryPluginWrapper;
	using Base = DiscoveryPlugin;
	using Registry = DiscoveryPluginRegistry;

	using FactoryHelper = internal::py::WrapperFactory<
		                      This,
		                      Base,
		                      Registry::FactoryFn,
		                      const DiscoveryOptions&>;

	DiscoveryPluginWrapper(const DiscoveryOptions& options)
		: DiscoveryPlugin(options)
	{
	}

	virtual ~DiscoveryPluginWrapper() = default;

	// -------------------------------------------------------------------------
	// DiscoveryPlugin::Initialize

 	bool
	default_Initialize(const pxr::UsdStageRefPtr& stage)
	{
		return Base::Initialize(stage);
	}

	bool
	Initialize(const pxr::UsdStageRefPtr& stage) override
	{
		return CallPythonOverride(this,

			"Initialize",

			// Default implementation
			&This::default_Initialize,

			// Args
			stage);
	}

	// -------------------------------------------------------------------------
	// DiscoveryPlugin::GetInitialPath

	pxr::SdfPath
	default_GetInitialPath() const
	{
		return Base::GetInitialPath();
	}

	pxr::SdfPath
	GetInitialPath() const override
	{
		return CallPythonOverrideWithResultHandler(this,

			"GetInitialPath",

			// Default implementation
			&This::default_GetInitialPath,

			// Python -> C++ result handler
			[&](py::object result) -> pxr::SdfPath
			{
				if (result.is_none())
				{
					return {};
				}
				return py::extract<pxr::SdfPath>(result)();
			});
	}

	// -------------------------------------------------------------------------
	// DiscoveryPlugin::CreateRootDefinition

	ExtAssetDefinition
	default_CreateRootDefinition(const pxr::UsdPrim& prim) const
	{
		return Base::CreateRootDefinition(prim);
	}

	ExtAssetDefinition
	CreateRootDefinition(const pxr::UsdPrim& prim) const override
	{
		return CallPythonOverrideWithResultHandler(this,

			"CreateRootDefinition",

			// Default implementation
			&This::default_CreateRootDefinition,

			// Python -> C++ result handler
			[&](py::object result) -> ExtAssetDefinition
			{
				if (result.is_none())
				{
					return {};
				}
				return py::extract<ExtAssetDefinition>(result)();
			},

			// Args
			prim);
	}

	// -------------------------------------------------------------------------
	// DiscoveryPlugin::IsAsset

	bool
	default_IsAsset(const pxr::UsdPrim& prim) const
	{
		return Base::IsAsset(prim);
	}

	bool
	IsAsset(const pxr::UsdPrim& prim) const override
	{
		return CallPythonOverride(this,

			"IsAsset",

			// Default implementation
			&This::default_IsAsset,

			// Args
			prim);
	}

	// -------------------------------------------------------------------------
	// DiscoveryPlugin::GetAssetDefinition

	ExtAssetDefinition
	default_GetAssetDefinition(const pxr::UsdPrim& prim) const
	{
		return Base::GetAssetDefinition(prim);
	}

	ExtAssetDefinition
	GetAssetDefinition(const pxr::UsdPrim& prim) const override
	{
		return CallPythonOverride(this,

			"GetAssetDefinition",

			// Default implementation
			&This::default_GetAssetDefinition,

			// Args
		    prim);
	}

	// -------------------------------------------------------------------------
	// DiscoveryPlugin::GetCurrentPrimPermutation

	PrimPermutation
	default_GetCurrentPrimPermutation(const pxr::UsdPrim& prim) const
	{
		return Base::GetCurrentPrimPermutation(prim);
	}

	PrimPermutation
	GetCurrentPrimPermutation(const pxr::UsdPrim& prim) const override
	{
		return CallPythonOverrideWithResultHandler(this,

			"GetCurrentPrimPermutation",

			// Default implementation
			&This::default_GetCurrentPrimPermutation,

			// Python -> C++ result handler
		    [&](py::object result) -> PrimPermutation
		    {
			    if (result.is_none())
			    {
					return {};
			    }
				return py::extract<PrimPermutation>(result)();
			},

			// Args
		    prim);
	}

	// -------------------------------------------------------------------------
	// DiscoveryPlugin::GetPrimPermutations

	std::vector<PrimPermutation>
	default_GetPrimPermutations(const pxr::UsdPrim& prim) const
	{
		return Base::GetPrimPermutations(prim);
	}

	std::vector<PrimPermutation>
	GetPrimPermutations(const pxr::UsdPrim& prim) const override
	{
		return CallPythonOverrideWithResultHandler(this,

			"GetPrimPermutations",

			// Default implementation
			&This::default_GetPrimPermutations,

			// Python -> C++ result handler
		    [&](py::object result) -> std::vector<PrimPermutation>
		    {
				if (result.is_none())
				{
					return {};
				}

				return py::extract<std::vector<PrimPermutation>>(result)();
			},

			// Args
		    prim);
	}

	// -------------------------------------------------------------------------
	// DiscoveryPlugin::PrunePrim

	bool
	default_PrunePrim(const pxr::UsdPrim& prim) const
	{
		return Base::PrunePrim(prim);
	}

	bool
	PrunePrim(const pxr::UsdPrim& prim) const override
	{
		return CallPythonOverride(this,

			"PrunePrim",

			// Default implementation
			&This::default_PrunePrim,

			// Args
		    prim);
	}

	// -------------------------------------------------------------------------
	// DiscoveryPlugin::GetContentCategoryForPrim

	pxr::TfToken
	default_GetContentCategoryForPrim(
		const pxr::UsdPrim& prim,
		const ExtAssetDefinition* defn) const
	{
		return Base::GetContentCategoryForPrim(prim, defn);
	}

	pxr::TfToken
	GetContentCategoryForPrim(
		const pxr::UsdPrim& prim,
		const ExtAssetDefinition* defn) const override
	{
		return CallPythonOverride(this,

			"GetContentCategoryForPrim",

			// Default implementation
			&This::default_GetContentCategoryForPrim,

			// Args
		    prim,
			defn);
	}

	// -------------------------------------------------------------------------
	// DiscoveryPlugin::GetOverrideCandidatePrimPaths

	std::optional<pxr::SdfPathSet>
	default_GetOverrideCandidatePrimPaths(
		const pxr::UsdPrim& assetPrim,
		const ExtAssetDefinition* defn) const
	{
		return Base::GetOverrideCandidatePrimPaths(assetPrim, defn);
	}

	std::optional<pxr::SdfPathSet>
	GetOverrideCandidatePrimPaths(
		const pxr::UsdPrim& assetPrim,
		const ExtAssetDefinition* defn) const override
	{
		return CallPythonOverrideWithResultHandler(this,

			"GetOverrideCandidatePrimPaths",

			// Default implementation
			&This::default_GetOverrideCandidatePrimPaths,

			// Return handler
		    [&](py::object result) -> std::optional<pxr::SdfPathSet>
		    {
				if (result.is_none())
				{
					return std::nullopt;
				}

				pxr::SdfPathSet ret;
				py::stl_input_iterator<pxr::SdfPath> begin(result), end;
				ret.insert(begin, end);
				return ret;
			},

			// Args
		    assetPrim,
			defn);
	}

	// -------------------------------------------------------------------------

	// Public accessor to protected get_override() so that it can
	// be called from VirtualDispatchHelper
	py::override GetOverride(const char* functionName) const
	{
		return this->get_override(functionName);
	}

	static DiscoveryPluginWrapper*
	Construct(uintptr_t wrapper)
	{
		return reinterpret_cast<DiscoveryPluginWrapper*>(wrapper);
	}

	static void
	Register(py::object cls, const std::string& pluginName)
	{
		// Prevent registration of the wrapped base class itself.
		//
		// The wrapper construction system expects a concrete Python subclass
		// instance to exist for virtual dispatch. Registering the wrapped
		// base DiscoveryPlugin type directly bypasses those assumptions and
		// can corrupt Boost.Python wrapper/converter state.
		py::object module = py::import("UsdPregen");
		py::object discoveryPluginClass = module.attr("DiscoveryPlugin");

		if (cls.ptr() == discoveryPluginClass.ptr())
		{
			TF_CODING_ERROR(
			    "Cannot register DiscoveryPlugin base class directly. "
				"Register a Python subclass instead."
			);
			return;
		}

		if (Registry::FactoryFn fn = FactoryHelper::MakeFactory(cls))
		{
			Registry::GetInstance().RegisterFactory(pluginName, fn);
		}
	}

	static py::object
	Create(const DiscoveryOptions& options)
	{
		DiscoveryPluginRefPtr discovery = Registry::GetInstance().Create(options);

		return internal::py::
			CreatePythonPluginObject<DiscoveryPluginWrapper>(discovery);
	}
};

// NOTE:
//
// We intentionally expose Initialize() to Python through a shim taking a
// generic Python object rather than binding the native
// `UsdStageRefPtr` signature directly.
//
// Directly exposing:
//
//     bool Initialize(const pxr::UsdStageRefPtr&)
//
// through Boost.Python caused USD Python converters to become corrupted
// or mismatched after calling the method on Python-backed plugin wrapper
// instances. After the call, unrelated USD bindings (such as
// UsdStage.GetPrimAtPath) would begin failing argument conversion checks.
//
// Manually extracting the stage object:
//
//     py::extract<pxr::UsdStage*>(pyStage)
//
// and reconstructing a UsdStageRefPtr avoids the issue and keeps the USD
// converter registry stable.
//
// The exact root cause is unclear, but appears related to interaction
// between Boost.Python polymorphic wrapper dispatch, shared pointer/refptr
// conversions, and USD's registered Python converters for UsdStage.
bool
_BaseInitialize(DiscoveryPlugin& plugin, py::object pyStage)
{
	pxr::TfPyLock lock;
	pxr::UsdStage* stage = py::extract<pxr::UsdStage*>(pyStage)();
	pxr::UsdStageRefPtr stageRef(stage);
	return plugin.DiscoveryPlugin::Initialize(stageRef);
}

struct TokensNamespace
{
};

} // anonymous namespace

#define WRAP_TOKEN(name)							    \
	.add_static_property(#name,		                    \
	py::make_function(+[]() -> std::string {	        \
		return DiscoveryPluginTokens::name.GetString(); \
	}))

void wrapDiscoveryPlugin()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	using This = DiscoveryPlugin;
	using Wrapper = DiscoveryPluginWrapper;

	py::register_ptr_to_python<DiscoveryPluginRefPtr>();

	py::object cls =
	py::class_<Wrapper, py::noncopyable>
		("DiscoveryPlugin", py::no_init)
		.def("__init__", py::make_constructor(&Wrapper::Construct))

		.def("Initialize",
			&_BaseInitialize,
			(py::arg("stage")))

		.def("GetInitialPath",
			&This::GetInitialPath,
			&Wrapper::default_GetInitialPath)

		.def("IsAsset",
			&This::IsAsset,
			&Wrapper::default_IsAsset,
			(py::arg("prim")))

		.def("GetAssetDefinition",
			&This::GetAssetDefinition,
			&Wrapper::default_GetAssetDefinition,
			(py::arg("prim")))

		.def("CreateRootDefinition",
			&This::CreateRootDefinition,
			&Wrapper::default_CreateRootDefinition,
			(py::arg("prim")))

		.def("GetPrimPermutations",
			&This::GetPrimPermutations,
			&Wrapper::default_GetPrimPermutations,
			py::return_value_policy<pxr::TfPySequenceToList>(),
			(py::arg("prim")))

		.def("GetCurrentPrimPermutation",
			&This::GetCurrentPrimPermutation,
			&Wrapper::default_GetCurrentPrimPermutation,
			(py::arg("prim")))

		.def("PrunePrim",
			&This::PrunePrim,
			&Wrapper::default_PrunePrim,
		    (py::arg("prim")))

		.def("GetContentCategoryForPrim",
			&This::GetContentCategoryForPrim,
			&Wrapper::default_GetContentCategoryForPrim,
			(py::arg("prim"),
			 py::arg("defn")))

		.def("GetOverrideCandidatePrimPaths",
			&This::GetOverrideCandidatePrimPaths,
			&Wrapper::default_GetOverrideCandidatePrimPaths,
			(py::arg("assetPrim"),
			 py::arg("defn")))

		.def("GetOptions", &This::GetOptions,
			py::return_internal_reference<>())

		.def("Register", &Wrapper::Register)
		.staticmethod("Register")

		.def("Create", &Wrapper::Create)
		.staticmethod("Create")
	;

	py::object tokensClass =
	py::class_<TokensNamespace>("Tokens", py::no_init)
		 WRAP_TOKEN(reservedMetadataPrefix)
		 WRAP_TOKEN(definitionPrefix)
		 WRAP_TOKEN(initialPrim)
	;

	cls.attr("Tokens") = tokensClass;
}

#endif // USE_USD_SDK
