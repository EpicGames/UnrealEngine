// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/extAssetDefinition.h"

#include "USDIncludesStart.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/external/boost/python/operators.hpp"
#include "pxr/base/tf/pyContainerConversions.h"
#include "pxr/base/tf/stringUtils.h"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

void wrapExtAssetDefinition()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	typedef ExtAssetDefinition This;

	py::class_<This>
		("ExtAssetDefinition")

		.def(py::init<>())

		.def(py::init<const std::string&,
				  const std::string&,
				  const pxr::SdfAssetPath&>(
			(py::arg("name"),
			 py::arg("version"),
			 py::arg("identifier"))))

		.def(py::init<const std::string&,
				  const std::string&,
				  const pxr::SdfAssetPath&,
				  const std::string&>(
			(py::arg("name"),
			 py::arg("version"),
			 py::arg("identifier"),
			 py::arg("customUniqueId"))))

		.def(py::init<const std::string&,
				  const std::string&,
				  const pxr::SdfAssetPath&,
				  const pxr::VtDictionary&>(
			(py::arg("name"),
			 py::arg("version"),
			 py::arg("identifier"),
			 py::arg("metadata"))))

		.def(py::init<const std::string&,
				  const std::string&,
				  const pxr::SdfAssetPath&,
				  const std::string&,
				  const pxr::VtDictionary&>(
			(py::arg("name"),
			 py::arg("version"),
			 py::arg("identifier"),
			 py::arg("customUniqueId"),
			 py::arg("metadata"))))

		.def("GetName", +[](const This& defn) -> std::string {
			return defn.GetName();
		})

		.def("GetVersion", +[](const This& defn) -> std::string {
			return defn.GetVersion();
		})

		.def("GetIdentifier", +[](const This& defn) -> pxr::SdfAssetPath {
			return defn.GetIdentifier();
		})

		.def("GetUniqueId", +[](const This& defn) -> std::string {
			return defn.GetUniqueId();
		})

		.def("GetMetadata", +[](const This& defn) -> py::object {
			const auto& metadata = defn.GetMetadata();
			return metadata ? py::object(*metadata) : py::object();
		})

		.def("HasCustomUniqueId", &This::HasCustomUniqueId)

		.def("HasMetadata", &This::HasMetadata)

		.def("IsValid", +[](const This& defn) -> bool {
			return defn.IsValid();
		})

		.def("HasSameFields", &This::HasSameFields)
		.staticmethod("HasSameFields")

		.def("__bool__", &This::operator bool)

		.def("__repr__", +[](const This& defn) -> std::string {
			std::string reason;
			if (defn.IsValid(&reason)) {
				return
					pxr::TfStringPrintf(
						"ExtAssetDefinition(name='%s', version='%s', identifier'%s', uniqueId='%s')"
						, defn.GetName().c_str()
						, defn.GetVersion().c_str()
						, defn.GetIdentifier().GetAuthoredPath().c_str()
						, defn.GetUniqueId().c_str()
					);
			}
			return pxr::TfStringPrintf("invalid definition - %s", reason.c_str());
		})

		.def("__hash__", +[](const This & defn) -> std::size_t {
			return std::hash<std::string>{}(defn.GetUniqueId());
		})

		.def(py::self == py::self)
		.def(py::self != py::self)
	;

	// Register conversion for python list <-> vector<const ExtAssetDefinition*>
	using Vec = std::vector<const ExtAssetDefinition*>;
	py::to_python_converter<Vec, pxr::TfPySequenceToPython<Vec>>();
	pxr::TfPyContainerConversions::from_python_sequence<
		std::vector<const ExtAssetDefinition*>,
		pxr::TfPyContainerConversions::
		variable_capacity_all_items_convertible_policy >();
}

#endif // USE_USD_SDK
