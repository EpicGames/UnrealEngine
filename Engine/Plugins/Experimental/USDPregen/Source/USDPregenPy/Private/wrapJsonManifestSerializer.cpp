// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/jsonManifestSerializer.h"
#include "UsdPregen/manifest.h"
#include "UsdPregen/manifestTypes.h"

#include "USDIncludesStart.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/base/tf/stringUtils.h"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

void wrapJsonManifestSerializer()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	using This = JsonManifestSerializer;

	py::class_<This, py::bases<ManifestSerializer>>
		("JsonManifestSerializer", py::init<>())

		.def("Serialize", +[](const This& serializer,
							 const Manifest& manifest,
							 ManifestPayload& payload) {
				return serializer.Serialize(manifest, payload);
			})
		.def("Deserialize", +[](const This& serializer,
								const ManifestPayload& payload) {
				return serializer.Deserialize(payload);
			})
		.def("GetSchemaName", +[]() {
				return This::GetSchemaName();
			})
		.staticmethod("GetSchemaName")
		.def("GetSchemaVersion", +[]() {
				return This::GetSchemaVersion();
			})
		.staticmethod("GetSchemaVersion")
		.def("Encoding", +[]() {
				return This::Encoding();
			})
		.staticmethod("Encoding")
		.def("FileExtension", +[]() {
				return This::FileExtension();
			})
		.staticmethod("FileExtension")
		.def("__repr__",
			+[](const This&) {
				return pxr::TfStringPrintf(
						   "JsonManifestSerializer(schema='%s', version=%d)",
						   This::GetSchemaName().c_str(),
						   This::GetSchemaVersion());
			})
		;
}

#endif // USE_USD_SDK
