// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/jsonStoragePlugin.h"

#include "UsdPregen/manifest.h"
#include "UsdPregen/manifestTypes.h"
#include "UsdPregen/storageOptions.h"
#include "UsdPregen/target.h"

#include "USDIncludesStart.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "USDIncludesEnd.h"

#include <memory>
#include <string>

PREGEN_NAMESPACE_USING_DIRECTIVE

void wrapJsonStoragePlugin()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	using This = JsonStoragePlugin;

	py::class_<This, JsonStoragePluginRefPtr, py::bases<StoragePlugin>, py::noncopyable>
		("JsonStoragePlugin", py::init<const StorageOptions&>(
		 (py::arg("options") = StorageOptions{})))

		.def("LoadManifestPayload", &This::LoadManifestPayload,
			(py::arg("targetUid")))

		.def("StoreManifestPayload", &This::StoreManifestPayload,
			(py::arg("targetUid"),
			 py::arg("payload")))

		.def("PersistManifestPayload", &This::PersistManifestPayload,
			(py::arg("targetUid")))

		.def("SerializeManifest", &This::SerializeManifest,
			(py::arg("manifest")))

		.def("DeserializeManifestPayload", &This::DeserializeManifestPayload,
			(py::arg("payload")))

		.def("GetPathForManifest", &This::GetPathForManifest,
			(py::arg("targetUid")))
	;
}

#endif // USE_USD_SDK
