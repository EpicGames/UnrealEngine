// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/manifest.h"
#include "UsdPregen/target.h"

#include "USDIncludesStart.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/base/tf/pyResultConversions.h"
#include "pxr/base/tf/stringUtils.h"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

void wrapManifest()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	using This = Manifest;

	py::class_<Product>("Product")
		.def_readwrite("upackagePath", &Product::upackagePath)
		.def_readwrite("uclass", &Product::uclass)
		.def_readwrite("unodeId", &Product::unodeId)
		.def_readwrite("usdPrimType", &Product::usdPrimType)
		.def_readwrite("usdPrimPath", &Product::usdPrimPath)
		.def("__repr__",
			+[](const Product& product) -> std::string {
				return pxr::TfStringPrintf(
						   "Product(upackagePath='%s', uclass='%s', unodeId='%s')",
						   product.upackagePath.c_str(),
						   product.uclass.c_str(),
						   product.unodeId.c_str());
			})
	;

	py::class_<This>("Manifest", py::init<>())
		.def("GetTargetUid", +[](const This& manifest) -> TargetUid {
				return manifest.GetTargetUid();
			})
		.def("AddProduct", &This::AddProduct)
		.def("GetProducts", +[](const This& manifest) -> std::vector<Product> {
				return manifest.GetProducts();
			},
			py::return_value_policy<pxr::TfPySequenceToList>())
		.def("SetTargetData", &This::SetTargetData)
		.def("GetTargetData", +[](const This& manifest) -> TargetDataRefPtr {
				return manifest.GetTargetData();
			})
		.def("IsValid", &This::IsValid)
		.def("__bool__", +[](const This& manifest) {
				return manifest.IsValid();
			})
		.def("__repr__",
			+[](const This& manifest) -> std::string {
				return pxr::TfStringPrintf(
						   "Manifest(targetUid='%s', numProducts=%zu)",
						   pxr::TfStringify(manifest.GetTargetUid()).c_str(),
						   manifest.GetProducts().size());
			})
	;
}
	
#endif // USE_USD_SDK
