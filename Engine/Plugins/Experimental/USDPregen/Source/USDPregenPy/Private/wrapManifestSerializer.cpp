// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/manifestSerializer.h"
#include "UsdPregen/manifest.h"
#include "UsdPregen/manifestTypes.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

void wrapManifestSerializer()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	using This = ManifestSerializer;

	py::class_<This, py::noncopyable>
		("ManifestSerializer", py::no_init)
		.def("GetEncoding",
			+[](const This& s) -> std::string {
				return s.GetEncoding();
			})
		.def("GetFileExtension",
			+[](const This& s) -> std::string {
				return s.GetFileExtension();
			})
		.def("Serialize",
			+[](const This& s,
				const Manifest& manifest,
				ManifestPayload& payload)
			{
				return s.Serialize(manifest, payload);
			})
		.def("Deserialize",
			+[](const This& s,
				const ManifestPayload& payload)
			{
				return s.Deserialize(payload);
			})
		.def("__repr__",
			+[](const This&) {
				return "ManifestSerializer()";
			})
		;
}

#endif // USE_USD_SDK
