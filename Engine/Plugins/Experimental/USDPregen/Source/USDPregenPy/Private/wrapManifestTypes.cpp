// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/manifestTypes.h"

#include "USDIncludesStart.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/base/tf/pyEnum.h"
#include "pxr/base/tf/stringUtils.h"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

namespace {

namespace py = PXR_BOOST_PYTHON_NAMESPACE;

void _wrapManifestPayload()
{
	using This = ManifestPayload;

	py::class_<This>("ManifestPayload")
		.def_readwrite("encoding", &This::encoding)

		.add_property("data",
			// getter
			+[](const This& payload)
			{
				const size_t size = payload.data.size();
				if (size > static_cast<size_t>(PY_SSIZE_T_MAX))
				{
					PyErr_SetString(PyExc_OverflowError, "payload too large for Python bytes");
					py::throw_error_already_set();
				}

				const char* buf = size > 0
					? reinterpret_cast<const char*>(payload.data.data())
					: nullptr;

				PyObject* pyBytes =
					PyBytes_FromStringAndSize(buf, static_cast<Py_ssize_t>(size));

				if (!pyBytes)
				{
					py::throw_error_already_set();
				}

				return py::object(py::handle<>(pyBytes));
			},

			// setter
			+[](This& payload, py::object obj)
			{
				Py_buffer view;

				if (PyObject_GetBuffer(obj.ptr(), &view, PyBUF_SIMPLE) != 0)
					py::throw_error_already_set();

				struct FPyBufferRelease
				{
					Py_buffer* View;
					~FPyBufferRelease()
					{
						if (View)
						{
							PyBuffer_Release(View);
						}
					}
				} release{ &view };

				constexpr Py_ssize_t MaxPayloadSize = 64 * 1024 * 1024;

				if (view.len < 0)
				{
					PyErr_SetString(PyExc_ValueError, "buffer length is negative");
					py::throw_error_already_set();
				}

				if (view.len > MaxPayloadSize)
				{
					PyErr_Format(
						PyExc_ValueError,
						"buffer too large (%zd bytes, max %zd)",
						view.len,
						MaxPayloadSize);
					py::throw_error_already_set();
				}

				const size_t size = static_cast<size_t>(view.len);

				payload.data.resize(size);
				if (size > 0)
				{
					memcpy(payload.data.data(), view.buf, size);
				}
			}
		)

		.def("__repr__", +[](const This& payload) -> std::string {
			return pxr::TfStringPrintf(
				"ManifestPayload(encoding='%s', size=%zu)",
				payload.encoding.c_str(),
				payload.data.size());
		})
	;
}

void _wrapManifestLoad()
{
	pxr::TfPyWrapEnum<ManifestLoadStatus>();

	using This = ManifestLoadResult;

	py::class_<This>("ManifestLoadResult")
		.def_readwrite("status", &This::status)
		.def_readwrite("payload", &This::payload)
		.def_readwrite("message", &This::message)

		.def("__bool__", +[](const This& r)
		{
			return r.status != ManifestLoadStatus::Error;
		})

		.def("__repr__", +[](const This& r) -> std::string
		{
			return pxr::TfStringPrintf(
				"ManifestLoadResult(status=%d, message='%s')",
				static_cast<int>(r.status),
				r.message.c_str());
		})
	;
}

void _wrapManifestSave()
{
	pxr::TfPyWrapEnum<ManifestSaveStatus>();

	using This = ManifestSaveResult;

	py::class_<This>("ManifestSaveResult")
		.def_readwrite("status", &This::status)
		.def_readwrite("message", &This::message)

		.def("__bool__", +[](const This& r)
		{
			return r.status != ManifestSaveStatus::Error;
		})

		.def("__repr__", +[](const This& r) -> std::string
		{
			return pxr::TfStringPrintf(
				"ManifestSaveResult(status=%d, message='%s')",
				static_cast<int>(r.status),
				r.message.c_str());
		})
	;
}

} // anonymous namespace

void wrapManifestTypes()
{
	_wrapManifestPayload();
	_wrapManifestLoad();
	_wrapManifestSave();
}

#endif // USE_USD_SDK
