// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/base/arch/export.h"
#include "pxr/external/boost/python.hpp"
#include "USDIncludesEnd.h"

#define _WRAP(x) ARCH_HIDDEN void wrap ## x (); wrap ## x ()

PXR_BOOST_PYTHON_MODULE(UsdPregen)
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	// ensure Sdf and Tf converters are registered
	try {
		py::import("pxr.Sdf");
	 	py::import("pxr.Tf");
	} catch (...) {
		PyErr_Print();
	}

	_WRAP(StoragePlugin);
	_WRAP(ManifestSerializer);

	_WRAP(AssetDefinitionRegistry);
	_WRAP(StorageOptions);
	_WRAP(JsonStoragePlugin);
	_WRAP(DiscoveryOptions);
	_WRAP(DiscoveryPlugin);
	_WRAP(ExtAssetDefinition);
	_WRAP(JsonManifestSerializer);
	_WRAP(Manifest);
	_WRAP(ManifestTypes);
	_WRAP(PermutationOps);
	_WRAP(PrimPermutation);
	_WRAP(SceneDiscovery);
	_WRAP(SceneTracker);
	_WRAP(Target);
	_WRAP(TrackedPrim);

}

#endif // USE_USD_SDK
