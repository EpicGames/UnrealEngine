// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/permutationOps.h"

#include "USDIncludesStart.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/base/tf/pyResultConversions.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/usd/prim.h"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

namespace
{

namespace py = PXR_BOOST_PYTHON_NAMESPACE;

void _wrapPermutationOp()
{
	using This = PermutationOp;

	py::class_<This, PermutationOpRefPtr, py::noncopyable>
		("PermutationOp", py::no_init)
		.def("GetUniqueId", &This::GetUniqueId)
		.def("Apply", &This::Apply)
	;
}

void _wrapUsdVariantPermutationOp()
{
	using This = UsdVariantPermutationOp;

	py::class_<This, UsdVariantPermutationOpRefPtr, py::bases<PermutationOp>, py::noncopyable>
		("UsdVariantPermutationOp", py::no_init)
		.def(py::init<const std::string&, const std::string&>(
			(py::arg("variantSet"),
			 py::arg("variant"))))
		.def("GetVariantSelection", &This::GetVariantSelection)
		.def("FromSerialized", &This::FromSerialized,
			(py::arg("variantSet"), py::arg("variant")))
		.staticmethod("FromSerialized")
	;
}

void _wrapUsdInheritPermutationOp()
{
	using This = UsdInheritPermutationOp;

	py::class_<This, UsdInheritPermutationOpRefPtr, py::bases<PermutationOp>, py::noncopyable>
		("UsdInheritPermutationOp", py::no_init)
		.def(py::init<const pxr::SdfPath&, pxr::SdfListOpType>(
			(py::arg("pathToInherit"),
			 py::arg("listOpType") = pxr::SdfListOpTypePrepended)))
		.def("GetPathToInherit", &This::GetPathToInherit,
			py::return_value_policy<py::return_by_value>())
		.def("GetListOpType", &This::GetListOpType)
		.def("FromSerialized", &This::FromSerialized,
			(py::arg("pathToInherit"),
			 py::arg("listOpType") = pxr::SdfListOpTypePrepended))
		.staticmethod("FromSerialized")
		;
}

void _wrapSchemaApplyPermutationOp()
{
	using This = SchemaApplyPermutationOp;

	py::class_<This, SchemaApplyPermutationOpRefPtr, py::bases<PermutationOp>, py::noncopyable>
		("SchemaApplyPermutationOp", py::no_init)
		.def(py::init<const std::string&>((py::arg("schemaName"))))
		.def("GetSchemaName", &This::GetSchemaName)
		.def("FromSerialized", &This::FromSerialized, (py::arg("schemaName")))
		.staticmethod("FromSerialized")
	;
}

} // anonymous namespace

void wrapPermutationOps()
{
	_wrapPermutationOp();
	_wrapUsdVariantPermutationOp();
	_wrapUsdInheritPermutationOp();
	_wrapSchemaApplyPermutationOp();
}

#endif // USE_USD_SDK
