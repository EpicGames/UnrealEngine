// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/primPermutation.h"

#include "USDIncludesStart.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/base/tf/pyContainerConversions.h"
#include "pxr/base/tf/pyResultConversions.h"
#include "pxr/base/tf/stringUtils.h"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

void wrapPrimPermutation()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	using This = PrimPermutation;

	py::class_<This>
		("PrimPermutation")

		.def(py::init<>())

		.def(py::init<const pxr::SdfPath&>((py::arg("path"))))

		.def("GetPath",
			 +[](const This& perm) -> pxr::SdfPath {
				return perm.GetPath();
			 })

		.def("GetUniqueId", &This::GetUniqueId)

		.def("AppendOp", &This::AppendOp)

		.def("GetOps",
			 +[](const This& perm) -> PermutationOpVector {
				 return perm.GetOps();
			 },
			 py::return_value_policy<pxr::TfPySequenceToList>())

		.def("GetConsumesDescendants", &This::GetConsumesDescendants)

		.def("SetConsumesDescendants", &This::SetConsumesDescendants)

		.def("IsEmpty", &This::IsEmpty)

		.def("__repr__",
			 +[](const This& perm) -> std::string {
				 return
					 pxr::TfStringPrintf(
						 "PrimPermutation(uid='%s')"
						 , perm.GetUniqueId().c_str()
					 );
			 })
	;

	// Register conversion for python list <-> vector<PrimPermutation>
	using Vec = std::vector<PrimPermutation>;
	py::to_python_converter<Vec, pxr::TfPySequenceToPython<Vec>>();
	pxr::TfPyContainerConversions::from_python_sequence<
		Vec, pxr::TfPyContainerConversions::
		variable_capacity_all_items_convertible_policy>();
}

#endif // USE_USD_SDK
