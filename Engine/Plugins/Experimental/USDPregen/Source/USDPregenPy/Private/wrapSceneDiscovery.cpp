// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/sceneDiscovery.h"

#include "UsdPregen/discoveryOptions.h"
#include "UsdPregen/target.h"
#include "UsdPregen/sceneTracker.h"

#include "USDIncludesStart.h"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/external/boost/python/extract.hpp"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

namespace
{

namespace py = PXR_BOOST_PYTHON_NAMESPACE;

py::object
_TraverseAndFindTargets(SceneDiscovery& traversal)
{
	SceneDiscovery::ResultMap results;
	bool success = traversal.TraverseAndFindTargets(results);
	py::dict py_results;
	for (auto const& [path, vec] : results)
	{
		py::list py_list;
		for (const TargetUid& uid : vec)
		{
			py_list.append(uid);
		}
		py_results[path] = py::tuple(py_list);
	}

	return py::make_tuple(success, py_results);
}

} // anonymous namespace

void wrapSceneDiscovery()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	typedef SceneDiscovery This;

	py::class_<This, py::noncopyable>
		("SceneDiscovery", py::no_init)
		.def(py::init<pxr::UsdStageRefPtr>(
			(py::arg("stage"))
		))
		.def(py::init<pxr::UsdStageRefPtr, DiscoveryOptions>(
	        (py::arg("stage"), py::arg("options"))
		))
		.def("GetTargetData", &This::GetTargetData)
		.def("TraverseAndFindTargets", &_TraverseAndFindTargets)
		.def("SaveDiscoveryData", &This::SaveDiscoveryData)
	;
}

#endif // USE_USD_SDK
