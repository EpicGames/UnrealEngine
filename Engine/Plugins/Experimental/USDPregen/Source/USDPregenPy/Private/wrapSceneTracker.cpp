// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/sceneTracker.h"

#include "UsdPregen/discoveryOptions.h"
#include "UsdPregen/target.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/pyFunction.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/external/boost/python/make_constructor.hpp"
#include "USDIncludesEnd.h"

#include <memory>
#include <string>

PREGEN_NAMESPACE_USING_DIRECTIVE

namespace
{

namespace py = PXR_BOOST_PYTHON_NAMESPACE;

std::shared_ptr<TrackedPrim>
_TrackedPrim_enter(std::shared_ptr<TrackedPrim> trackedPrim)
{
	return trackedPrim;
}

bool
_TrackedPrim_exit(std::shared_ptr<TrackedPrim> trackedPrim,
	py::object exc_type, py::object exc, py::object tb)
{
	// Returning false means do not suppress exceptions
	return false;
}

} // anonymous namespace

void wrapTrackedPrim()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	typedef TrackedPrim This;

	py::class_<This, std::shared_ptr<TrackedPrim>, py::noncopyable>
		("TrackedPrim", py::no_init)
		.def("HasUnprocessedPermutations", &This::HasUnprocessedPermutations)
		.def("PrepareNextPermutation", &This::PrepareNextPermutation)

		// context manager support
		.def("__enter__", &_TrackedPrim_enter)
		.def("__exit__",  &_TrackedPrim_exit)
	;
}

namespace
{

namespace py = PXR_BOOST_PYTHON_NAMESPACE;

py::object
_StartTrackingPrim(SceneTracker& tracker, const pxr::UsdPrim& prim)
{
	py::object py_self{
		std::make_shared<TrackedPrim>(tracker.StartTrackingPrim(prim))
	};
	return py_self;
}

} // anonymous namespace

void wrapSceneTracker()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	typedef SceneTracker This;

	pxr::TfPyFunctionFromPython<void(const pxr::SdfPath&, const TargetUid&)>();

	py::scope s = py::class_<This, SceneTrackerRefPtr, py::noncopyable>
		("SceneTracker", py::no_init)
		.def("__init__", py::make_constructor(
			+[](const pxr::UsdStageRefPtr& stage) {
				SceneTrackerRefPtr tracker = SceneTracker::Create(stage);
				if (!tracker)
				{
					PyErr_SetString(PyExc_RuntimeError, "Failed to create SceneTracker");
					py::throw_error_already_set();
				}
				return tracker;
			},
			py::default_call_policies(),
			(py::arg("stage"))))
		.def("__init__", py::make_constructor(
			+[](const pxr::UsdStageRefPtr& stage, const DiscoveryOptions& options) {
				SceneTrackerRefPtr tracker = SceneTracker::Create(stage, options);
				if (!tracker)
				{
					PyErr_SetString(PyExc_RuntimeError, "Failed to create SceneTracker");
					py::throw_error_already_set();
				}
				return tracker;
			},
			py::default_call_policies(),
			(py::arg("stage"),
			 py::arg("options"))))
		.def("GetTargetData", &This::GetTargetData)
		.def("HasErrors", &This::HasErrors)
		.def("SaveDataLayer", &This::SaveDataLayer)
		.def("SetTargetCreatedCallback", &This::SetTargetCreatedCallback)
		.def("RemoveTargetCreatedCallback", &This::RemoveTargetCreatedCallback)
		.def("StartTrackingPrim", &_StartTrackingPrim)
	;
}

#endif // USE_USD_SDK
