// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/storageOptions.h"

#include "USDIncludesStart.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "USDIncludesEnd.h"

#include <set>
#include <string>

PREGEN_NAMESPACE_USING_DIRECTIVE

namespace {

namespace py = PXR_BOOST_PYTHON_NAMESPACE;

void
_Options_setattr(py::object obj, py::str name, py::object value)
{
	// Helper to prevent invalid field names from going
	// undetected when set via python.
	static const std::set<std::string> fieldNames
	{
		"storagePluginName",
		"manifestDir",
		"packageSubPathTemplate"
	};

	const std::string nameStr = py::extract<std::string>(name);

	if (fieldNames.count(nameStr))
	{
		// Delegate to default behavior
		py::object object_type(py::handle<>(py::borrowed((PyObject*)&PyBaseObject_Type)));
		py::object base_setattr = object_type.attr("__setattr__");
		base_setattr(obj, name, value);
	}
	else
	{
		PyErr_SetString(PyExc_AttributeError,
			("Unknown attribute: " + nameStr).c_str());
		py::throw_error_already_set();
	}
}

} // anonymous namespace

void wrapStorageOptions()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	typedef StorageOptions This;

	py::class_<This>("StorageOptions")
		.def(py::init<>())

		.def_readwrite("storagePluginName",
			&This::storagePluginName)

		.def_readwrite("manifestDir",
			&This::manifestDir)

		.def_readwrite("packageSubPathTemplate",
			&This::packageSubPathTemplate)

		.def("__repr__", &This::DumpToString)

		.def("__setattr__", &_Options_setattr)
		.def(py::self == py::self)
		.def(py::self != py::self)
	;
}

#endif // USE_USD_SDK
