// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/storagePlugin.h"

#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/manifest.h"
#include "UsdPregen/manifestTypes.h"
#include "UsdPregen/storagePluginRegistry.h"
#include "UsdPregen/target.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/pyLock.h"
#include "pxr/base/tf/pyPolymorphic.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/external/boost/python/make_constructor.hpp"
#include "USDIncludesEnd.h"

#include "wrapperHelpers.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

PXR_NAMESPACE_USING_DIRECTIVE

namespace
{

namespace py = PXR_BOOST_PYTHON_NAMESPACE;

class StoragePluginWrapper
	: public StoragePlugin
	, public pxr::TfPyPolymorphic<StoragePlugin>
	, protected internal::py::VirtualDispatchHelper
{
public:

	using This = StoragePluginWrapper;
	using Base = StoragePlugin;
	using Registry = StoragePluginRegistry;

	using FactoryHelper = internal::py::WrapperFactory<
		                      This,
		                      Base,
		                      Registry::FactoryFn,
		                      const StorageOptions&>;

	StoragePluginWrapper(const StorageOptions& options)
		: StoragePlugin(options)
	{
	}

	virtual ~StoragePluginWrapper() = default;

	// -------------------------------------------------------------------------
	// StoragePlugin::Initialize

	bool
	default_Initialize()
	{
		return StoragePlugin::Initialize();
	}

	bool
	Initialize() override
	{
		return CallPythonOverride(this,

			"Initialize",

			// Default implementation
			&This::default_Initialize);
	}

	// -------------------------------------------------------------------------
	// StoragePlugin::Shutdown

	bool
	default_Shutdown()
	{
		return StoragePlugin::Shutdown();
	}

	bool
	Shutdown() override
	{
		return CallPythonOverride(this,

			"Shutdown",

			// Default implementation
			&This::default_Shutdown);
	}

	// -------------------------------------------------------------------------
	// StoragePlugin::StoreManifestPayload

	ManifestSaveResult
	StoreManifestPayload(
		const TargetUid& targetUid,
		const ManifestPayload& payload) override
	{
		return CallPythonPureVirtual<ManifestSaveResult>(this,

			"StoreManifestPayload",

			// Args
			targetUid,
			payload);
	}

	// -------------------------------------------------------------------------
	// StoragePlugin::PersistManifestPayload

	ManifestSaveResult
	default_PersistManifestPayload(const TargetUid& targetUid)
	{
		return StoragePlugin::PersistManifestPayload(targetUid);
	}

	ManifestSaveResult
	PersistManifestPayload(const TargetUid& targetUid) override
	{
		return CallPythonOverride(this,

			"PersistManifestPayload",

			// Default implementation
			&This::default_PersistManifestPayload,

			// Args
			targetUid);
	}

	// -------------------------------------------------------------------------
	// StoragePlugin::LoadManifestPayload

	ManifestLoadResult
	LoadManifestPayload(const TargetUid& targetUid) override
	{
		return CallPythonPureVirtual<ManifestLoadResult>(this,

			"LoadManifestPayload",

			// Args
			targetUid);
	}

	// -------------------------------------------------------------------------
	// StoragePlugin::SerializeManifest

	ManifestPayload
	SerializeManifest(const Manifest& manifest) override
	{
		return CallPythonPureVirtual<ManifestPayload>(this,

			"SerializeManifest",

			// Args
			manifest);
	}

	// -------------------------------------------------------------------------
	// StoragePlugin::DeserializeManifestPayload

	Manifest
	DeserializeManifestPayload(const ManifestPayload& payload) override
	{
		return CallPythonPureVirtual<Manifest>(this,
				   "DeserializeManifestPayload",
				   payload);
	}

	// -------------------------------------------------------------------------
	// StoragePlugin::GetNameForUAsset

	std::string
	GetNameForUAsset(const TargetUid& targetUid,
					 const std::vector<const ExtAssetDefinition*>& defns,
					 const std::string& assetType) override
	{
		return CallPythonPureVirtual<std::string>(this,

			"GetNameForUAsset",

			// Args
			targetUid,
			defns,
			assetType);
	}

	// -------------------------------------------------------------------------
	// StoragePlugin::GetPackageSubPathForUAsset

	std::string
	GetPackageSubPathForUAsset(const TargetUid& targetUid,
							   const std::vector<const ExtAssetDefinition*>& defns,
							   const std::string& assetType) override
	{
		return CallPythonPureVirtual<std::string>(this,

			"GetPackageSubPathForUAsset",

			// Args
			targetUid,
			defns,
			assetType);
	}

	// -------------------------------------------------------------------------
	// StoragePlugin::GetPathForManifest

	std::string
	GetPathForManifest(const TargetUid& targetUid) override
	{
		return CallPythonPureVirtual<std::string>(this,

			"GetPathForManifest",

			// Args
			targetUid);
	}

	// -------------------------------------------------------------------------

	// Public accessor to protected get_override() so that it can
	// be called from VirtualDispatchHelper
	py::override GetOverride(const char* functionName) const
	{
		return this->get_override(functionName);
	}

	static StoragePluginWrapper*
	Construct(uintptr_t wrapper)
	{
		return reinterpret_cast<StoragePluginWrapper*>(wrapper);
	}

	static void
	Register(py::object cls, const std::string& pluginName)
	{
		// Prevent registration of the wrapped base class itself.
		//
		// The wrapper construction system expects a concrete Python subclass
		// instance to exist for virtual dispatch. Registering the wrapped
		// base StoragePlugin type directly bypasses those assumptions and
		// can corrupt Boost.Python wrapper/converter state.
		py::object module = py::import("UsdPregen");
		py::object storagePluginClass = module.attr("StoragePlugin");

		if (cls.ptr() == storagePluginClass.ptr())
		{
			TF_CODING_ERROR(
			    "Cannot register StoragePlugin base class directly. "
				"Register a Python subclass instead."
			);
			return;
		}

		if (Registry::FactoryFn fn = FactoryHelper::MakeFactory(cls))
		{
			Registry::GetInstance().RegisterFactory(pluginName, fn);
		}
	}

	static py::object
	Create(const StorageOptions& options)
	{
		StoragePluginRefPtr storage = Registry::GetInstance().Create(options);

		return internal::py::
			CreatePythonPluginObject<StoragePluginWrapper>(storage);
	}
};

static std::string
_ResolvePackageSubPathTemplate(
	const std::string& templateStr,
	const TargetUid& targetUid,
	const std::vector<const ExtAssetDefinition*>& definitions,
	const std::string& assetType,
	py::dict substitutions)
{
	std::unordered_map<std::string, std::string> map;

	py::list keys = substitutions.keys();
	for (Py_ssize_t i = 0; i < py::len(keys); ++i)
	{
		std::string key = py::extract<std::string>(keys[i]);

		std::string value = py::extract<std::string>(
						        substitutions[keys[i]]);
		map[key] = value;
	}

	return StoragePlugin::ResolvePackageSubPathTemplate(
		       templateStr,
			   targetUid,
			   definitions,
			   assetType,
			   map);
}

} // anonymous namespace

void
wrapStoragePlugin()
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	using This = StoragePlugin;
	using Wrapper = StoragePluginWrapper;

	py::register_ptr_to_python<StoragePluginRefPtr>();

	py::class_<Wrapper, py::noncopyable>
		("StoragePlugin", py::no_init)

		.def("__init__", py::make_constructor(&Wrapper::Construct))

		.def("Initialize",
			&StoragePlugin::Initialize,
			&StoragePluginWrapper::default_Initialize)

		.def("Shutdown",
			&StoragePlugin::Shutdown,
			&StoragePluginWrapper::default_Shutdown)

		.def("LoadManifestPayload",
			py::pure_virtual(&This::LoadManifestPayload),
			(py::arg("targetUid")))

		.def("StoreManifestPayload",
			py::pure_virtual(&StoragePlugin::StoreManifestPayload),
			(py::arg("targetUid"),
			 py::arg("payload")))

		.def("PersistManifestPayload",
			&StoragePlugin::PersistManifestPayload,
			&StoragePluginWrapper::default_PersistManifestPayload,
			(py::arg("targetUid")))

		.def("SerializeManifest",
			py::pure_virtual(&StoragePlugin::SerializeManifest),
			(py::arg("manifest")))

		.def("DeserializeManifestPayload",
			py::pure_virtual(&This::DeserializeManifestPayload),
			(py::arg("payload")))

		.def("GetNameForUAsset",
			py::pure_virtual(&This::GetNameForUAsset),
			(py::arg("targetUid"),
			 py::arg("definitions"),
			 py::arg("assetType")))

		.def("GetPackageSubPathForUAsset",
			py::pure_virtual(&This::GetPackageSubPathForUAsset),
			(py::arg("targetUid"),
			 py::arg("definitions"),
			 py::arg("assetType")))

		.def("GetPathForManifest",
			py::pure_virtual(&This::GetPathForManifest),
			(py::arg("targetUid")))

		.def("GetOptions", &This::GetOptions,
			py::return_internal_reference<>())

		.def("DefaultPackageSubPathTemplate", &This::DefaultPackageSubPathTemplate,
			py::return_value_policy<py::copy_const_reference>())
		.staticmethod("DefaultPackageSubPathTemplate")

		.def("ResolvePackageSubPathTemplate", &_ResolvePackageSubPathTemplate)
		.staticmethod("ResolvePackageSubPathTemplate")

		.def("Register", &Wrapper::Register,
			(py::arg("cls"),
			 py::arg("pluginName")))
		.staticmethod("Register")

		.def("Create", &Wrapper::Create,
			(py::arg("options")))
		.staticmethod("Create")
	;
}

#endif // USE_USD_SDK
