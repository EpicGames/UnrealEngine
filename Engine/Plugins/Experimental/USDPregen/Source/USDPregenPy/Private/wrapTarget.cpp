// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "UsdPregen/target.h"
#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/permutationOps.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/pyResultConversions.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/external/boost/python.hpp"
#include "pxr/external/boost/python/class.hpp"
#include "pxr/external/boost/python/operators.hpp"
#include "USDIncludesEnd.h"

PREGEN_NAMESPACE_USING_DIRECTIVE

namespace
{

namespace py = PXR_BOOST_PYTHON_NAMESPACE;

void _wrapTargetUid()
{
	typedef TargetUid This;

	py::class_<This>
		("TargetUid")
		.def(py::init<>())
		.def(py::init<const std::string&>(
			(py::arg("definitionUid"))))
		.def(py::init<const std::string&, const std::string&>(
			(py::arg("definitionUid"),
			 py::arg("permutationUid"))))
		.def("GetDefinitionUid", &This::GetDefinitionUid)
		.def("GetPermutationUid", &This::GetPermutationUid)
		.def("HasPermutationUid", &This::HasPermutationUid)
		.def("IsValid", &This::IsValid)
		.def("__bool__", &This::operator bool)
		.def("__repr__", +[](const This& targetUid) -> std::string {
			return
				pxr::TfStringPrintf(
					"UsdPregen.TargetUid('%s')"
					, pxr::TfStringify(targetUid).c_str());
		})
		.def("__str__", +[](const This& targetUid) -> std::string {
			return pxr::TfStringify(targetUid);
		})
		.def("__hash__", +[](const This & targetUid) -> std::size_t {
			return std::hash<std::string>{}(pxr::TfStringify(targetUid));
		})
		.def(py::self == py::self)
		.def(py::self != py::self)
		.def(py::self < py::self)
	;
}

void _wrapTargetDefinitionEntry()
{
	typedef TargetDefinitionEntry This;

	py::class_<This>
		("TargetDefinitionEntry", py::no_init)
		.def("GetDefinition", &This::GetDefinition,
			py::return_internal_reference<>())
		.def("GetScenePath", +[](const This& target) -> pxr::SdfPath {
				return target.GetScenePath();
			})
		.def("GetPermutationOps", +[](const This& target) {
				py::list result;
				for (const PermutationOpRefPtr& op : target.GetPermutationOps())
				{
					result.append(op);
				}
				return result;
			})
	;
}

void _wrapTargetData()
{
	typedef TargetData This;

	py::class_<This, TargetDataRefPtr>
		("TargetData")
		.def(py::init<>())
		.def("GetUniqueId", &This::GetUniqueId)
		.def("NumDefinitionEntries", &This::NumDefinitionEntries)
		.def("GetDefinitionEntry", &This::GetDefinitionEntry,
			py::return_internal_reference<>())
		.def("GetDefinitionEntries", +[](const This& targetData) {
				py::list result;
				for (const TargetDefinitionEntry& entry : targetData.GetDefinitionEntries())
				{
					result.append(entry);
				}
				return result;
			})
		.def("GetDependencies", +[](const This& targetData) {
				py::list result;
				for (const TargetUid& uid : targetData.GetDependencies())
				{
					result.append(uid);
				}
				return result;
			})
		.def("GetEncapsulatedDefinitionPaths", +[](const This& targetData) {
				py::object pySet(py::handle<>(PySet_New(nullptr)));
				for (const pxr::SdfPath& path : targetData.GetEncapsulatedDefinitionPaths())
				{
					PySet_Add(pySet.ptr(), py::object(path).ptr());
				}
				return pySet;
			})
		.def("GetUnencapsulatedDefinitionPaths", +[](const This& targetData) {
				py::object pySet(py::handle<>(PySet_New(nullptr)));
				for (const pxr::SdfPath& path : targetData.GetUnencapsulatedDefinitionPaths())
				{
					PySet_Add(pySet.ptr(), py::object(path).ptr());
				}
				return pySet;
			})
		.def("GetPermutationOverlay", &This::GetPermutationOverlay)
		.def("IsValid", &This::IsValid)
		.def("__bool__", &This::operator bool)
		.def("__repr__", +[](const This& data) -> std::string {
				if (data.IsValid()) {
					return pxr::TfStringPrintf("UsdPregen.TargetData()");
				}
				return "invalid target";
			})
	;
}

} // anonymous namespace

void wrapTarget()
{
	_wrapTargetUid();
	_wrapTargetDefinitionEntry();
	_wrapTargetData();
}

#endif // USE_USD_SDK

