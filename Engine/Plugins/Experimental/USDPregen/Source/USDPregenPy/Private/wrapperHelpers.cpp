// Copyright Epic Games, Inc. All Rights Reserved.

#if USE_USD_SDK

#include "wrapperHelpers.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/tf/diagnostic.h"
#include "pxr/base/tf/stringUtils.h"
#include "USDIncludesEnd.h"

#include <cstdarg>

PXR_NAMESPACE_USING_DIRECTIVE

PREGEN_NAMESPACE_OPEN_SCOPE

namespace internal::py
{

void
PostRuntimeError(const std::string& msg)
{
	TF_RUNTIME_ERROR("%s", msg.c_str());
}

void
PostWarning(const std::string& msg)
{
	TF_WARN("%s", msg.c_str());
}

std::string
GetPythonTraceback()
{
	PyObject *type = nullptr, *value = nullptr, *traceback = nullptr;
	PyErr_Fetch(&type, &value, &traceback);

	if (!type)
	{
		return "No active Python exception.";
	}

	// Normalizes the exception (e.g., converts type/value into a proper exception instance)
	PyErr_NormalizeException(&type, &value, &traceback);
	if (traceback && value)
	{
		PyException_SetTraceback(value, traceback);
	}

	std::string result;

	// Use Python's built-in traceback formatting but with a single execution string
	PyObject* tracebackModule = PyImport_ImportModule("traceback");
	if (tracebackModule)
	{
		PyObject* formattedList = PyObject_CallMethod(
								      tracebackModule,
									  "format_exception",
									  "O",
									  value);
		if (formattedList)
		{
			PyObject* emptyStr = PyUnicode_FromString("");
			PyObject* joined = emptyStr ? PyUnicode_Join(emptyStr, formattedList) : nullptr;

			if (joined)
			{
				if (const char* utf8 = PyUnicode_AsUTF8(joined))
				{
					result = utf8;
				}
				Py_DECREF(joined);
			}
			Py_XDECREF(emptyStr);
			Py_DECREF(formattedList);
		}
		Py_DECREF(tracebackModule);
	}

	// Fallback to basic string conversion if traceback module failed
	if (result.empty() && value)
	{
		PyObject* strObj = PyObject_Str(value);
		if (strObj)
		{
			if (const char* utf8 = PyUnicode_AsUTF8(strObj))
			{
				result = utf8;
			}
			Py_DECREF(strObj);
		}
	}

	Py_XDECREF(type);
	Py_XDECREF(value);
	Py_XDECREF(traceback);

	return result;
}

} // namespace internal::py

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK

