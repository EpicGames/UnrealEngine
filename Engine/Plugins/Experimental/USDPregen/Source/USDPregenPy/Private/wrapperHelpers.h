// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

#include "UsdPregen/manifestTypes.h"
#include "UsdPregen/target.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "pxr/base/arch/demangle.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/tf/pyLock.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/external/boost/python.hpp"
#include "USDIncludesEnd.h"

#include <string>

PREGEN_NAMESPACE_OPEN_SCOPE

namespace internal::py
{

void
PostRuntimeError(const std::string& msg);

void
PostWarning(const std::string& msg);

std::string
GetPythonTraceback();

// -----------------------------------------------------------------------------
// Helpers for configuring manifest load/save error messages.
//
// Defined at namespace scope rather than as member templates of
// VirtualDispatchHelper because explicit specializations of a member template
// inside the enclosing class are not permitted by the C++ standard (MSVC
// accepts this as an extension; Clang/GCC reject it).

template <typename T>
inline void
_SetErrorMessage(T& /*result*/, const std::string& /*msg*/)
{
}

template <>
inline void
_SetErrorMessage<ManifestLoadResult>(
	ManifestLoadResult& result,
	const std::string& msg)
{
	if (result.status == ManifestLoadStatus::Error
		&& result.message.empty())
	{
		result.message = msg;
	}
}

template <>
inline void
_SetErrorMessage<ManifestSaveResult>(
	ManifestSaveResult& result,
	const std::string& msg)
{
	if (result.status == ManifestSaveStatus::Error
		&& result.message.empty())
	{
		result.message = msg;
	}
}

/// Shared helper functionality for Boost.Python virtual dispatch wrappers.
///
/// This helper centralizes:
///   - GIL acquisition
///   - Python override lookup
///   - fallback/default dispatch
///   - Python exception handling
///   - improved runtime diagnostics for bad return types
///
/// Wrapper implementations inherit this protectedly:
///
///   class FooWrapper
///       : public Foo
///       , public pxr::TfPyPolymorphic<Foo>
///       , protected VirtualDispatchHelper
///
class VirtualDispatchHelper
{
protected:

	/// Converts a Python object into the requested C++ return type using
	/// boost::python::extract.
	template <typename ReturnType>
	static ReturnType
	DefaultResultHandler(
	    const PXR_BOOST_PYTHON_NAMESPACE::object& result)
	{
		namespace py = PXR_BOOST_PYTHON_NAMESPACE;
		return py::extract<ReturnType>(result)();
	}
	
	/// Calls an optional Python override and falls back to the supplied default
	/// C++ implementation when no override exists. The provided result handler
	/// function will be called to marshall the return type from Python to C++
	template <
		typename WrapperType,
		typename DefaultFn,
		typename ResultFn,
		typename... Args>
	static auto
	CallPythonOverrideWithResultHandler(
		WrapperType* wrapper,
		const char* functionName,
		DefaultFn defaultFn,
		ResultFn&& resultFn,
		Args&&... args)
	{
		namespace py = PXR_BOOST_PYTHON_NAMESPACE;

		using ReturnType = decltype((wrapper->*defaultFn)(args...));

		pxr::TfPyLock lock;

		py::object result;

		auto CallDefault = [&]() -> ReturnType
		{
			return (wrapper->*defaultFn)(args...);
		};

		// Call the override function, or the default function if the override
		// doesn't exist. If we hit an exception, route it through our handler
		// so that we can log more diagnostics output than a regular boost
		// python exception.
		try
		{
			if (py::override func = wrapper->GetOverride(functionName))
			{
				result = py::object(func(args...));
			}
			else
			{
				return CallDefault();
			}
		}
		catch (const py::error_already_set&)
		{
			const std::string msg = _BuildExceptionMsg(
									    *wrapper, functionName, args...);

			// Emit a TF_RUNTIME_ERROR for the exception so that it can be
			// detected by TfErrorMark if needed (rather than just stderr)
			PostRuntimeError(msg);

			// If the result type is an errored manifest save or load request,
			// add our exception message to the message field of the result
			// object.
			ReturnType errorResult;
			_SetErrorMessage(errorResult, msg);

			return errorResult;
		}

		// Attempt to call the result handler function for the return type. We
		// try this in a separate block to the user code above so we can emit a
		// more helpful runtime error message when there is an issue with
		// the return type.
		try
		{
			return resultFn(result);
		}
		catch (const py::error_already_set&)
		{
			_EmitResultConversionError<ReturnType>(
				*wrapper,
				functionName,
				result.ptr() ? Py_TYPE(result.ptr())->tp_name : "unknown_type",
				args...);

			return ReturnType{};
		}
	}

	/// Calls an optional Python override using the default result conversion
	/// handler for the deduced return type.
	template <
		typename WrapperType,
		typename DefaultFn,
		typename... Args>
	static auto
	CallPythonOverride(
	    WrapperType* wrapper,
		const char* functionName,
		DefaultFn defaultFn,
		Args&&... args)
	{
		using ReturnType = decltype((wrapper->*defaultFn)(args...));

		return CallPythonOverrideWithResultHandler(
		           wrapper,
				   functionName,
				   defaultFn,
				   DefaultResultHandler<ReturnType>,
				   args...);
	}

	/// Calls a required Python override using a custom result conversion
	/// handler and emits diagnostics when the override fails or returns an
	/// invalid type.
	template <
		typename ReturnType,
		typename WrapperType,
		typename ResultFn,
		typename... Args>
	static ReturnType
	CallPythonPureVirtualWithResultHandler(
	    WrapperType* wrapper,
		const char* functionName,
		ResultFn&& resultFn,
		Args&&... args)
	{
		namespace py = PXR_BOOST_PYTHON_NAMESPACE;

		pxr::TfPyLock lock;

		py::object result;

		// Call the override function, or the default function if the override
		// doesn't exist. If we hit an exception, route it through our handler
		// so that we can log more diagnostics output than a regular boost
		// python exception.
		try
		{
			if (py::override func = wrapper->GetOverride(functionName))
			{
				result = py::object(func(args...));
			}
			else
			{
				PyErr_SetString(PyExc_TypeError, "Failed to find function");
				throw py::error_already_set();
			}
		}
		catch (const py::error_already_set&)
		{
			const std::string msg = _BuildExceptionMsg(
								        *wrapper, functionName, args...);

			// Emit a TF_RUNTIME_ERROR for the exception so that it can be
			// detected by TfErrorMark if needed (rather than just stderr)
			PostRuntimeError(msg);

			// If the result type is an errored manifest save/load request
			// add the exception message to the message field of the result
			// object.
			ReturnType errorResult;
			_SetErrorMessage(errorResult, msg);

			return errorResult;
		}

		// Attempt to call the result handler function for the return type. We
		// try this in a separate block to the user code above so we can emit a
		// more helpful runtime error message when there is an issue with
		// the return type.
		try
		{
			return resultFn(result);
		}
		catch (const py::error_already_set&)
		{
			_EmitResultConversionError<ReturnType>(
			    *wrapper,
				functionName,
				result.ptr() ? Py_TYPE(result.ptr())->tp_name : "unknown_type",
				args...);

			return ReturnType{};
		}
	}

	/// Calls a required Python override using the default result
	/// handler for the specified return type.
	template <
		typename ReturnType,
		typename WrapperType,
		typename... Args>
	static ReturnType
	CallPythonPureVirtual(
	    WrapperType* wrapper,
		const char* functionName,
		Args&&... args)
	{
		return CallPythonPureVirtualWithResultHandler<ReturnType>(
		           wrapper,
				   functionName,
				   DefaultResultHandler<ReturnType>,
				   args...);
	}

	/// Returns the Python class name associated with the wrapped instance.
	template <typename WrapperType>
	static std::string
	GetPythonClassName(const WrapperType& wrapper)
	{
		namespace py = PXR_BOOST_PYTHON_NAMESPACE;

		std::string clsName = typeid(WrapperType).name();

		if (PyObject* pyPtr = py::detail::wrapper_base_::get_owner(wrapper))
		{
			py::handle<> handle(py::borrowed(pyPtr));
			py::object obj(handle);
			clsName = py::extract<std::string>(
				obj.attr("__class__").attr("__name__"))();
		}

		return clsName;
	}

private:

	// -------------------------------------------------------------------------
	// Helpers for extracting common argument types such as prims and targets

	template <typename T>
	static const pxr::UsdPrim*
	_GetPrimPtr(const T&)
	{
		return nullptr;
	}

	static const pxr::UsdPrim*
	_GetPrimPtr(const pxr::UsdPrim& prim)
	{
		return &prim;
	}

	template <typename T>
	static pxr::UsdStageRefPtr
	_GetStagePtr(const T&)
	{
		return nullptr;
	}

	static pxr::UsdStageRefPtr
	_GetStagePtr(const pxr::UsdStageRefPtr& stage)
	{
		return stage;
	}

	template <typename T>
	static const TargetUid*
	_GetTargetUidPtr(const T&)
	{
		return nullptr;
	}

	static const TargetUid*
	_GetTargetUidPtr(const TargetUid& targetUid)
	{
		return &targetUid;
	}

	static const pxr::UsdPrim*
	_FindPrimInArgs()
	{
		return nullptr;
	}

	template <typename First, typename... Rest>
	static const pxr::UsdPrim*
	_FindPrimInArgs(const First& first, const Rest&... rest)
	{
		if (const pxr::UsdPrim* prim = _GetPrimPtr(first))
		{
			return prim;
		}
		return _FindPrimInArgs(rest...);
	}

	static pxr::UsdStageRefPtr
	_FindStageInArgs()
	{
		return nullptr;
	}

	template <typename First, typename... Rest>
	static pxr::UsdStageRefPtr
	_FindStageInArgs(const First& first, const Rest&... rest)
	{
		if (pxr::UsdStageRefPtr stage = _GetStagePtr(first))
		{
			return stage;
		}
		return _FindStageInArgs(rest...);
	}

	static const TargetUid*
	_FindTargetUidInArgs()
	{
		return nullptr;
	}

	template <typename First, typename... Rest>
	static const TargetUid*
	_FindTargetUidInArgs(const First& first, const Rest&... rest)
	{
		if (const TargetUid* targetUid = _GetTargetUidPtr(first))
		{
			return targetUid;
		}
		return _FindTargetUidInArgs(rest...);
	}

	// Returns contextual diagnostic information extracted from the wrapped
	// function arguments.
	//
	// Generic Boost.Python runtime errors and conversion failures can often be
	// cryptic and lack sufficient context to identify which scene element or
	// target caused the failure. This helper attempts to extract commonly used
	// contextual types from the argument list (such as UsdPrim, UsdStageRefPtr,
	// or TargetUid) and formats additional diagnostic information to improve
	// runtime error reporting for the majority of wrapped plugin methods.
	template <typename... Args>
	static std::string
	_GetArgContext(const Args&... args)
	{
		// Resolve prim context first, if any.
		if (const pxr::UsdPrim* prim = _FindPrimInArgs(args...))
		{
			if (prim->GetStage())
			{
				const pxr::SdfPath& primPath = prim->GetPrimPath();

				const pxr::SdfLayerRefPtr rootLayer =
					prim->GetStage()->GetRootLayer();

				const pxr::ArResolvedPath resolvedPath =
					rootLayer->GetResolvedPath();

				return pxr::TfStringPrintf(
					" while processing prim <%s> in stage @%s@",
					primPath.GetText(),
					resolvedPath.GetPathString().c_str());
			}
		}

		// Next resolve stage context, if any.
		if (pxr::UsdStageRefPtr stage = _FindStageInArgs(args...))
		{
			const pxr::SdfLayerRefPtr rootLayer =
				stage->GetRootLayer();

			const pxr::ArResolvedPath resolvedPath =
				rootLayer->GetResolvedPath();

			return pxr::TfStringPrintf(
				" while processing stage @%s@",
				resolvedPath.GetPathString().c_str());
		}

		// Lastly try TargetUid context, if any.
		if (const TargetUid* targetUid = _FindTargetUidInArgs(args...))
		{
			return pxr::TfStringPrintf(
				" while processing target uid (%s)",
				targetUid->GetString().c_str());
		}

		return {};
	}

	// -------------------------------------------------------------------------
	// Error message helpers.

	template <
	    typename WrapperType,
	    typename... Args>
	static std::string
	_BuildExceptionMsg(
	    const WrapperType& wrapper,
		const char* functionName,
		const Args&... args)
	{
		const std::string argContext = _GetArgContext(args...);

		const std::string traceback = GetPythonTraceback();

		const std::string msg =
			pxr::TfStringPrintf(
			    "Exception occurred from Python function %s.%s%s ...\n%s",
				GetPythonClassName(wrapper).c_str(),
				functionName,
				argContext.c_str(),
				traceback.c_str());

		return msg;
	}

	template <
	    typename ReturnType,
	    typename WrapperType,
	    typename... Args>
	static void
	_EmitResultConversionError(
		const WrapperType& wrapper,
		const char* functionName,
		const char* actualType,
		const Args&... args)
	{
		const std::string argContext = _GetArgContext(args...);

		const std::string returnTypeName =
		    pxr::ArchGetDemangled(typeid(ReturnType).name());

		const std::string traceback = GetPythonTraceback();

		const std::string msg =
			pxr::TfStringPrintf(
				"Exception occurred from Python function %s.%s%s - "
				"expected return type <class '%s'> got <class '%s'> ...\n%s",
				GetPythonClassName(wrapper).c_str(),
				functionName,
				argContext.c_str(),
				returnTypeName.c_str(),
				actualType,
				traceback.c_str());

		PostRuntimeError(msg);
	}
};

template <
	typename WrapperType,
	typename BaseType,
	typename FactoryFnType,
	typename... CtorArgs>
class WrapperFactory
{
public:

	BaseType*
	operator()(CtorArgs... args) const
	{
		namespace py = PXR_BOOST_PYTHON_NAMESPACE;

		if (!_cls)
		{
			return nullptr;
		}

		auto wrapper = new WrapperType(std::forward<CtorArgs>(args)...);

		pxr::TfPyLock pyLock;

		py::object instance(_cls(reinterpret_cast<uintptr_t>(wrapper)));

		py::incref(instance.ptr());

		py::detail::initialize_wrapper(instance.ptr(), wrapper);

		return wrapper;
	}

	static FactoryFnType
	MakeFactory(PXR_BOOST_PYTHON_NAMESPACE::object cls)
	{
		return WrapperFactory(cls);
	}

private:

	explicit
	WrapperFactory(PXR_BOOST_PYTHON_NAMESPACE::object cls)
		: _cls(cls)
	{
	}

	PXR_BOOST_PYTHON_NAMESPACE::object _cls;
};

template <
	typename WrapperType,
	typename RefPtrType>
static PXR_BOOST_PYTHON_NAMESPACE::object
CreatePythonPluginObject(const RefPtrType& plugin)
{
	namespace py = PXR_BOOST_PYTHON_NAMESPACE;

	if (!plugin)
	{
		return py::object();
	}

	// Check whether the created plugin instance is a Python-backed wrapper.
	//
	// Pure native C++ plugins registered with the registry will not derive
	// from WrapperType and therefore cannot provide a Python owner object
	// for virtual dispatch.
	auto* wrapper = dynamic_cast<WrapperType*>(plugin.get());

	// Fallback for native C++ plugin instances.
	//
	// Returning the raw pointer exposes the shared_ptr-managed C++ instance
	// directly to Python. The instance remains fully usable from Python,
	// but naturally does not support Python virtual overrides because no
	// Python subclass object exists.
	if (!wrapper)
	{
		return py::object(py::ptr(plugin.get()));
	}

	// Retrieve the Python owner object associated with the wrapper.
	//
	// TfPyPolymorphic / Boost.Python maintain a mapping between the wrapped
	// C++ instance and the originating Python subclass instance used for
	// virtual dispatch.
	PyObject* owner =
		py::detail::wrapper_base_::get_owner(*wrapper);

	// Fallback if the wrapper has no associated Python owner object.
	//
	// This should normally not occur for properly constructed Python-backed
	// plugins, but can happen if wrapper initialization was bypassed or
	// partially constructed. In this case we again expose the underlying
	// shared_ptr-managed C++ instance directly.
	if (!owner)
	{
		const std::string msg = pxr::TfStringPrintf(
			"Python wrapper instance '%s' has no associated "
			"Python owner object. Falling back to direct C++ exposure.",
			pxr::ArchGetDemangled(typeid(*wrapper).name()).c_str());

		PostWarning(msg);

		return py::object(py::ptr(plugin.get()));
	}

	py::object obj(py::handle<>(py::borrowed(owner)));

	// Retain shared ownership of the plugin instance for the lifetime of
	// the Python object. Without this, the temporary shared_ptr created in
	// this function would be destroyed on return, potentially leaving the
	// Python wrapper referencing a destroyed C++ object.
	obj.attr("_plugin_base_instance") = plugin;

	return obj;
}

} // namespace internal::py

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
