// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDPregenPy : ModuleRules
	{
		public USDPregenPy(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseRTTI = true;
			bLegalToDistributeObjectCode = true;
			bUseUnity = false;
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"USDPregenCore",
					"UnrealUSDWrapper"
				}
			);
			
			bool bIsUsdSdkEnabled = UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
			if (bIsUsdSdkEnabled && (Target.Type == TargetType.Editor || Target.Platform == UnrealTargetPlatform.Win64))
			{
				PrivateDependencyModuleNames.Add("Python3");
			}
			
			string PlatformStr = Target.Platform.ToString();
			string PluginBinariesDir = Path.Combine(PluginDirectory, "Binaries", PlatformStr);
			string PythonSitePackagesDir = Path.Combine(PluginDirectory, "Content", "Python", PlatformStr, "Lib", "site-packages");
			
			// TODO Figure out a robust way to move/rename the resulting module binary into the 
			// site packages directory below. For now it's handled by the init_unreal.py script
			if (!Directory.Exists(PythonSitePackagesDir))
			{
				try { Directory.CreateDirectory(PythonSitePackagesDir); }
				catch (Exception InException)
				{
					Console.WriteLine("UsdPregenPy: Failed to add Python site-packages directory: " + InException.Message);
				}
			}
		}
	}
}

