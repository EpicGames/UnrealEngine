// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenManifestAsset.h"

#include "UsdPregenWrappers/ExtAssetDefinition.h"
#include "UsdPregenWrappers/Manifest.h"
#include "UsdPregenWrappers/Target.h"

#include "UsdWrappers/SdfPath.h"

namespace UE::UsdPregen
{
	struct FProduct;
}

using namespace UE::UsdPregen;

FUsdPregenProductEntry::FUsdPregenProductEntry(const UE::UsdPregen::FProduct& InProduct)
	: UPackagePath(InProduct.UPackagePath)
	, UClass(InProduct.UClass)
	, UNodeId(InProduct.UNodeId)
	, UsdPrimType(InProduct.UsdPrimType)
	, UsdPrimPath(InProduct.UsdPrimPath)
{
}

UE::UsdPregen::FProduct FUsdPregenProductEntry::ToWrapper() const
{
	UE::UsdPregen::FProduct Result;
	Result.UPackagePath = UPackagePath;
	Result.UClass = UClass;
	Result.UNodeId = UNodeId;
	Result.UsdPrimType = UsdPrimType;
	Result.UsdPrimPath = UsdPrimPath;
	return Result;
}

bool UUsdPregenManifestAsset::IsValidManifest() const
{
	// Definitions are mandatory: a manifest without them can't reconstruct
	// the originating TargetData and is therefore not a valid round-trip
	// payload.
	return !Uid.DefinitionUid.IsEmpty()
		&& Products.Num() > 0
		&& Definitions.Num() > 0;
}

void UUsdPregenManifestAsset::FromWrapper(const UE::UsdPregen::FManifest& InManifest)
{
	Uid = FUsdPregenTargetUidEntry{};
	Products.Reset();
	Dependencies.Reset();
	Definitions.Reset();

	if (!InManifest)
	{
		return;
	}

	const UE::UsdPregen::FTargetUid TargetUid = InManifest.GetTargetUid();
	if (TargetUid)
	{
		Uid.DefinitionUid = TargetUid.GetDefinitionUid();
		Uid.PermutationUid = TargetUid.GetPermutationUid();
	}

	const TArray<UE::UsdPregen::FProduct> WrapperProducts = InManifest.GetProducts();
	Products.Reserve(WrapperProducts.Num());

	for (const UE::UsdPregen::FProduct& Product : WrapperProducts)
	{
		Products.Add(FUsdPregenProductEntry{ Product });
	}

	const UE::UsdPregen::FTargetData TargetData = InManifest.GetTargetData();
	if (!TargetData)
	{
		return;
	}

	// Walk the originating TargetDefinitionEntry stack and emit one definition entry
	// per unique definition (deduped by UniqueId, first occurrence wins).
	// The TargetDefinitionEntry's scene path and permutation ops fold directly onto
	// the matching definition entry.
	const TArray<UE::UsdPregen::FTargetDefinitionEntry> WrapperInfos = TargetData.GetDefinitionEntries();

	TSet<FString> SeenDefinitions;

	for (const UE::UsdPregen::FTargetDefinitionEntry& Info : WrapperInfos)
	{
		const UE::UsdPregen::FExtAssetDefinition Defn = Info.GetDefinition();
		if (!Defn)
		{
			continue;
		}

		const FString DefnUid = Defn.GetUniqueId();
		if (SeenDefinitions.Contains(DefnUid))
		{
			continue;
		}
		SeenDefinitions.Add(DefnUid);

		const UE::UsdPregen::FExtAssetDefinitionSnapshot Snap
			= UE::UsdPregen::FExtAssetDefinitionSnapshot::From(Defn);

		FUsdPregenDefinitionEntry WrappedDefn;
		WrappedDefn.UniqueId = DefnUid;
		WrappedDefn.Name = Snap.Name;
		WrappedDefn.Version = Snap.Version;
		WrappedDefn.IdentifierAuthored = Snap.IdentifierAuthored;
		WrappedDefn.IdentifierResolved = Snap.IdentifierResolved;
		WrappedDefn.bHasCustomUniqueId = Snap.bHasCustomUniqueId;
		WrappedDefn.Metadata = Snap.Metadata;
		WrappedDefn.ScenePath = Info.GetScenePath().GetString();

		const TArray<UE::UsdPregen::FPermutationOp> Ops = Info.GetPermutationOps();
		WrappedDefn.PermutationOps.Reserve(Ops.Num());
		for (const UE::UsdPregen::FPermutationOp& Op : Ops)
		{
			FUsdPregenSerializedOpEntry WrappedOp;
			WrappedOp.TypeName = Op.TypeName;
			WrappedOp.Args.Reserve(Op.Opargs.Num());
			for (const UE::UsdPregen::FPermutationOpArg& Arg : Op.Opargs)
			{
				FUsdPregenSerializedOpArgEntry WrappedArg;
				WrappedArg.Name = Arg.Name;
				WrappedArg.TypeName = Arg.TypeName;
				WrappedArg.Value = Arg.Value;
				WrappedOp.Args.Add(MoveTemp(WrappedArg));
			}
			WrappedDefn.PermutationOps.Add(MoveTemp(WrappedOp));
		}

		Definitions.Add(MoveTemp(WrappedDefn));
	}

	const TArray<UE::UsdPregen::FTargetUid> Deps = TargetData.GetDependencies();
	Dependencies.Reserve(Deps.Num());
	for (const UE::UsdPregen::FTargetUid& Dep : Deps)
	{
		FUsdPregenTargetUidEntry WrappedDep;
		WrappedDep.DefinitionUid = Dep.GetDefinitionUid();
		WrappedDep.PermutationUid = Dep.GetPermutationUid();
		Dependencies.Add(MoveTemp(WrappedDep));
	}
}

UE::UsdPregen::FManifest UUsdPregenManifestAsset::ToWrapper() const
{
	UE::UsdPregen::FManifest Result;

	for (const FUsdPregenProductEntry& Product : Products)
	{
		Result.AddProduct(Product.ToWrapper());
	}

	if (Uid.DefinitionUid.IsEmpty())
	{
		// Without a uid the manifest cannot be associated with a target.
		// Leave the manifest in its default (invalid) state; IsValid() on
		// the wrapper will reflect this.
		return Result;
	}

	const UE::UsdPregen::FTargetUid TargetUid = Uid.PermutationUid.IsEmpty()
		? UE::UsdPregen::FTargetUid{ Uid.DefinitionUid }
		: UE::UsdPregen::FTargetUid{ Uid.DefinitionUid, Uid.PermutationUid };

	// Re-register definition snapshots into the global registry. Idempotent;
	// conflicting UIDs surface a warning from the registry itself.
	for (const FUsdPregenDefinitionEntry& Reflected : Definitions)
	{
		UE::UsdPregen::FExtAssetDefinitionSnapshot Snapshot;
		Snapshot.UniqueId = Reflected.UniqueId;
		Snapshot.Name = Reflected.Name;
		Snapshot.Version = Reflected.Version;
		Snapshot.IdentifierAuthored = Reflected.IdentifierAuthored;
		Snapshot.IdentifierResolved = Reflected.IdentifierResolved;
		Snapshot.bHasCustomUniqueId = Reflected.bHasCustomUniqueId;
		Snapshot.Metadata = Reflected.Metadata;
		Snapshot.RegisterIntoRegistry();
	}

	UE::UsdPregen::FTargetDataBuilder Builder{ TargetUid };

	// Reconstruct one TargetDefinitionEntry per definition entry. The fold from the
	// originating TargetDefinitionEntry stack onto definitions was 1:1 (deduped by
	// UniqueId) so this is the inverse of FromWrapper's emission loop.
	for (const FUsdPregenDefinitionEntry& Reflected : Definitions)
	{
		TArray<UE::UsdPregen::FPermutationOp> Ops;
		Ops.Reserve(Reflected.PermutationOps.Num());
		for (const FUsdPregenSerializedOpEntry& WrappedOp : Reflected.PermutationOps)
		{
			UE::UsdPregen::FPermutationOp Op;
			Op.TypeName = WrappedOp.TypeName;
			Op.Opargs.Reserve(WrappedOp.Args.Num());
			for (const FUsdPregenSerializedOpArgEntry& WrappedArg : WrappedOp.Args)
			{
				UE::UsdPregen::FPermutationOpArg Arg;
				Arg.Name = WrappedArg.Name;
				Arg.TypeName = WrappedArg.TypeName;
				Arg.Value = WrappedArg.Value;
				Op.Opargs.Add(MoveTemp(Arg));
			}
			Ops.Add(MoveTemp(Op));
		}

		Builder.AddInfo(Reflected.UniqueId,
		                UE::FSdfPath{*Reflected.ScenePath},
		                Ops);
	}

	TArray<UE::UsdPregen::FTargetUid> Deps;
	Deps.Reserve(Dependencies.Num());
	for (const FUsdPregenTargetUidEntry& Dep : Dependencies)
	{
		Deps.Add(UE::UsdPregen::FTargetUid{ Dep.DefinitionUid, Dep.PermutationUid });
	}
	Builder.SetDependencies(Deps);

	Result.SetTargetData(Builder.Build());

	return Result;
}
