// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenUObjectStorageModule.h"

#include "USDPregenUObjectStoragePlugin.h"

#include "UsdPregenWrappers/StoragePluginRegistry.h"

#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

void FUSDPregenUObjectStorageModule::StartupModule()
{
	auto RegisterStoragePlugin = []()
		{
			const FString PluginName = TEXT("uobject_storage");

			UE::UsdPregen::FStoragePluginRegistry Registry = UE::UsdPregen::FStoragePluginRegistry::GetInstance();

			const bool bRegistered = Registry.RegisterFactory(
				PluginName,
				[](const FPregenStorageOptions& Options) -> TSharedRef<UE::UsdPregen::IStoragePlugin, ESPMode::ThreadSafe>
				{
					return MakeShared<UE::UsdPregen::FUsdPregenUObjectStoragePlugin, ESPMode::ThreadSafe>(Options);
				}
			);

			ensureMsgf(
				bRegistered,
				TEXT("Failed to register USD Pregen storage plugin '%s'"),
				*PluginName
			);
		};

	// Defer registry access until engine init is complete: USD's TfSingleton /
	// TfStaticData / token machinery is not reliably in steady state across DLLs
	// during plugin StartupModule(), and first-touching StoragePluginRegistry
	// too early can crash inside its constructor.
	if (GEngine)
	{
		RegisterStoragePlugin();
	}
	else
	{
		OnPostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda(
			[this, RegisterStoragePlugin]()
			{
				RegisterStoragePlugin();

				FCoreDelegates::GetOnPostEngineInit().Remove(this->OnPostEngineInitHandle);
				this->OnPostEngineInitHandle.Reset();
			});
	}
}

void FUSDPregenUObjectStorageModule::ShutdownModule()
{
	if (OnPostEngineInitHandle.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(OnPostEngineInitHandle);
		OnPostEngineInitHandle.Reset();
	}
}

IMPLEMENT_MODULE(FUSDPregenUObjectStorageModule, USDPregenUObjectStorage)