// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenUObjectStoragePlugin.h"

#include "USDPregenManifestAsset.h"
#include "USDPregenSettings.h"

#include "UsdPregenWrappers/AssetDefinitionRegistry.h"
#include "UsdPregenWrappers/ExtAssetDefinition.h"
#include "UsdPregenWrappers/Manifest.h"
#include "UsdPregenWrappers/ManifestTypes.h"
#include "UsdPregenWrappers/Target.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/Char.h"
#include "Misc/PackageName.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"

namespace UE::UsdPregen
{
	namespace
	{
		FString SanitizePackageSegment(const FString& InSegment)
		{
			FString Result;
			Result.Reserve(InSegment.Len());

			for (TCHAR Char : InSegment)
			{
				if (FChar::IsAlnum(Char) || Char == TEXT('_'))
				{
					Result.AppendChar(Char);
				}
				else
				{
					Result.AppendChar(TEXT('_'));
				}
			}

			if (Result.IsEmpty())
			{
				Result = TEXT("Item");
			}

			return Result;
		}
	}	 // namespace

	FUsdPregenUObjectStoragePlugin::FUsdPregenUObjectStoragePlugin()
		: FUsdPregenUObjectStoragePlugin(FPregenStorageOptions{})
	{
	}

	FUsdPregenUObjectStoragePlugin::FUsdPregenUObjectStoragePlugin(const FPregenStorageOptions& Options)
		: ContentBasePath(GetDefault<UUSDPregenSettings>()->ImportContentPath.Path)
	{
		ContentBasePath.TrimStartAndEndInline();
		while (ContentBasePath.EndsWith(TEXT("/")))
		{
			ContentBasePath.LeftChopInline(1, EAllowShrinking::No);
		}

		// Template resolution order: per-instance options first, then the
		// project setting, then the framework default.
		PackageSubPathTemplate = Options.PackageSubPathTemplate;
		if (PackageSubPathTemplate.IsEmpty())
		{
			PackageSubPathTemplate = GetDefault<UUSDPregenSettings>()->PackageSubPathTemplate;
		}
		if (PackageSubPathTemplate.IsEmpty())
		{
			PackageSubPathTemplate = IStoragePlugin::DefaultPackageSubPathTemplate();
		}
	}

	FManifestLoadResult FUsdPregenUObjectStoragePlugin::LoadManifestPayload(const FTargetUid& TargetUid)
	{
		FManifestLoadResult Result;
		Result.Status = EManifestLoadStatus::Error;

		if (!TargetUid)
		{
			Result.Message = TEXT("Invalid target uid.");
			return Result;
		}

		UUsdPregenManifestAsset* ManifestAsset = LoadManifestAsset(TargetUid);
		if (!ManifestAsset)
		{
			Result.Status = EManifestLoadStatus::DoesNotExist;
			return Result;
		}

		FManifest Manifest = ManifestAsset->ToWrapper();
		if (!Manifest)
		{
			Result.Message = TEXT("Failed to convert manifest asset to wrapper manifest.");
			return Result;
		}

		Result.Payload = SerializeManifest(Manifest);
		if (Result.Payload.Encoding.IsEmpty() || Result.Payload.Data.IsEmpty())
		{
			Result.Message = TEXT("Failed to serialize manifest asset payload.");
			return Result;
		}

		Result.Status = EManifestLoadStatus::Loaded;
		return Result;
	}

	FManifestSaveResult FUsdPregenUObjectStoragePlugin::StoreManifestPayload(
		const FTargetUid& TargetUid,
		const FManifestPayload& Payload
	)
	{
		FManifestSaveResult Result;
		Result.Status = EManifestSaveStatus::Error;

		if (!TargetUid || Payload.Encoding.IsEmpty() || Payload.Data.IsEmpty())
		{
			Result.Message = TEXT("Invalid target uid and/or manifest payload.");
			return Result;
		}

		if (Payload.Encoding != GetPayloadEncoding())
		{
			Result.Message = FString::Printf(
				TEXT("Unsupported manifest encoding (%s)"),
				*Payload.Encoding
			);
			return Result;
		}

		const FString PackageName = GetPathForManifest(TargetUid);
		if (PackageName.IsEmpty())
		{
			Result.Message = TEXT("Failed to resolve manifest storage path.");
			return Result;
		}

		if (UUsdPregenManifestAsset* Existing = LoadManifestAsset(TargetUid))
		{
			Result.Status = EManifestSaveStatus::NotSaved;
			Result.Message = FString::Printf(
				TEXT("Found existing manifest UAsset for target '%s' at '%s' (looked up via '%s'); skipping store"),
				*TargetUid.GetString(),
				*Existing->GetPathName(),
				*MakeManifestObjectPath(TargetUid)
			);
			return Result;
		}

		FManifest Manifest = DeserializeManifestPayload(Payload);
		if (!Manifest)
		{
			Result.Message = TEXT("Failed to deserialize manifest payload.");
			return Result;
		}

		const FString AssetName = FPackageName::GetLongPackageAssetName(PackageName);

		UPackage* Package = CreatePackage(*PackageName);
		if (!Package)
		{
			Result.Message = FString::Printf(
				TEXT("Failed to create package (%s)"),
				*PackageName
			);
			return Result;
		}

		UUsdPregenManifestAsset* ManifestAsset = NewObject<UUsdPregenManifestAsset>(
			Package,
			*AssetName,
			RF_Public | RF_Standalone
		);

		if (!ManifestAsset)
		{
			Result.Message = FString::Printf(
				TEXT("Failed to create manifest asset (%s)"),
				*AssetName
			);
			return Result;
		}

		ManifestAsset->FromWrapper(Manifest);
		Package->MarkPackageDirty();

#if WITH_EDITOR
		FAssetRegistryModule::AssetCreated(ManifestAsset);

		Result.Status = EManifestSaveStatus::Saved;
		return Result;
#else
		Result.Message = TEXT("Saving UObject-backed manifests requires editor support.");
		return Result;
#endif
	}

	FManifestSaveResult FUsdPregenUObjectStoragePlugin::PersistManifestPayload(const FTargetUid& TargetUid)
	{
		FManifestSaveResult Result;
		Result.Status = EManifestSaveStatus::NotSaved;

		if (!TargetUid)
		{
			Result.Message = TEXT("Invalid target uid.");
			Result.Status = EManifestSaveStatus::Error;
			return Result;
		}

		const FString TargetUidStr = TargetUid.GetString();

		const FString PackageName = GetPathForManifest(TargetUid);
		if (PackageName.IsEmpty())
		{
			Result.Status = EManifestSaveStatus::Error;
			Result.Message = FString::Printf(
				TEXT("GetPathForManifest returned empty for target '%s' (ContentBasePath='%s')"),
				*TargetUidStr,
				*ContentBasePath
			);
			return Result;
		}

		UPackage* Package = FindPackage(nullptr, *PackageName);
		if (!Package)
		{
			Result.Status = EManifestSaveStatus::Error;
			Result.Message = FString::Printf(
				TEXT("FindPackage returned null for manifest package '%s' (target '%s'); StoreManifestPayload was expected to have created it earlier in this import"),
				*PackageName,
				*TargetUidStr
			);
			return Result;
		}
		if (!Package->IsDirty())
		{
			Result.Status = EManifestSaveStatus::Error;
			Result.Message = FString::Printf(
				TEXT("Manifest package '%s' (target '%s') was not dirty; nothing to persist (was it saved by another path or never modified?)"),
				*PackageName,
				*TargetUidStr
			);
			return Result;
		}

		const FString Filename = FPackageName::LongPackageNameToFilename(
			PackageName,
			FPackageName::GetAssetPackageExtension()
		);

		FSavePackageArgs Args;
		Args.TopLevelFlags = RF_Public | RF_Standalone;
		Args.SaveFlags = SAVE_None;
		Args.Error = GError;

		if (UPackage::SavePackage(Package, nullptr, *Filename, Args))
		{
			Result.Status = EManifestSaveStatus::Saved;
		}
		else
		{
			Result.Status = EManifestSaveStatus::Error;
			Result.Message = FString::Printf(
				TEXT("UPackage::SavePackage returned false for manifest package '%s' (target '%s', filename '%s')"),
				*PackageName,
				*TargetUidStr,
				*Filename
			);
		}
		return Result;
	}

	FManifestPayload FUsdPregenUObjectStoragePlugin::SerializeManifest(const FManifest& Manifest)
	{
		FManifestPayload Result;

		if (!Manifest)
		{
			return Result;
		}

		UUsdPregenManifestAsset* TempAsset = NewObject<UUsdPregenManifestAsset>(GetTransientPackage());
		if (!TempAsset)
		{
			return Result;
		}

		TempAsset->FromWrapper(Manifest);

		Result.Encoding = GetPayloadEncoding();
		FObjectWriter Writer(TempAsset, Result.Data);

		if (Result.Data.IsEmpty())
		{
			Result.Encoding.Reset();
		}

		return Result;
	}

	FManifest FUsdPregenUObjectStoragePlugin::DeserializeManifestPayload(const FManifestPayload& Payload)
	{
		if (Payload.Encoding != GetPayloadEncoding() || Payload.Data.IsEmpty())
		{
			return FManifest{};
		}

		UUsdPregenManifestAsset* TempAsset = NewObject<UUsdPregenManifestAsset>(GetTransientPackage());
		if (!TempAsset)
		{
			return FManifest{};
		}

		FObjectReader Reader(TempAsset, Payload.Data);
		return TempAsset->ToWrapper();
	}

	FString FUsdPregenUObjectStoragePlugin::GetNameForUAsset(
		const FTargetUid& TargetUid,
		const TArray<FExtAssetDefinition>& Definitions,
		const FString& AssetType
	)
	{
		// Do not modify the name (use the default Interchange name).
		return {};
	}

	FString FUsdPregenUObjectStoragePlugin::GetPackageSubPathForUAsset(
		const FTargetUid& TargetUid,
		const TArray<FExtAssetDefinition>& Definitions,
		const FString& AssetType
	)
	{
		// The path layout is driven by PackageSubPathTemplate (resolved in
		// the constructor from the per-instance options, the project
		// setting, and finally the framework default). The shared helper
		// keeps substitution semantics consistent across every storage
		// plugin.
		//
		// Note that the returned value is a sub-path, the parent of which
		// is the folder the user selected during import.

		if (!TargetUid || Definitions.IsEmpty())
		{
			return {};
		}

		return IStoragePlugin::ResolvePackageSubPathTemplate(
			PackageSubPathTemplate,
			TargetUid,
			Definitions,
			AssetType
		);
	}

	FString FUsdPregenUObjectStoragePlugin::GetPathForManifest(const FTargetUid& TargetUid)
	{
		if (!TargetUid
			|| ContentBasePath.IsEmpty()
			|| !FPackageName::IsValidLongPackageName(ContentBasePath))
		{
			return {};
		}

		// GetPackageSubPathForUAsset only uses the leaf definition, so a
		// single registry lookup is enough; we don't need the full chain.
		const FExtAssetDefinition LeafDef =
			FAssetDefinitionRegistry::GetInstance().GetDefinition(TargetUid.GetDefinitionUid());
		if (!LeafDef)
		{
			return {};
		}
		const TArray<FExtAssetDefinition> Defs{LeafDef};

		const FString SubPath = GetPackageSubPathForUAsset(TargetUid, Defs, FString{});

		return ContentBasePath / SubPath / MakeManifestAssetName(TargetUid);
	}

	FString FUsdPregenUObjectStoragePlugin::GetPayloadEncoding()
	{
		return TEXT("application/x-ue-usdpregen-uobject");
	}

	FString FUsdPregenUObjectStoragePlugin::MakeManifestAssetName(const FTargetUid& TargetUid) const
	{
		FString Result = SanitizePackageSegment(TargetUid.GetDefinitionUid());

		if (TargetUid.HasPermutationUid())
		{
			Result += TEXT("__");
			Result += SanitizePackageSegment(TargetUid.GetPermutationUid());
		}

		Result += TEXT("_Manifest");
		return Result;
	}

	FString FUsdPregenUObjectStoragePlugin::MakeManifestObjectPath(const FTargetUid& TargetUid)
	{
		const FString PackageName = GetPathForManifest(TargetUid);
		if (PackageName.IsEmpty())
		{
			return {};
		}
		const FString AssetName = FPackageName::GetLongPackageAssetName(PackageName);
		return PackageName + TEXT(".") + AssetName;
	}

	UUsdPregenManifestAsset* FUsdPregenUObjectStoragePlugin::LoadManifestAsset(const FTargetUid& TargetUid)
	{
		if (!TargetUid)
		{
			return nullptr;
		}

		const FString ObjectPath = MakeManifestObjectPath(TargetUid);
		if (ObjectPath.IsEmpty())
		{
			return nullptr;
		}

		UUsdPregenManifestAsset* Asset = Cast<UUsdPregenManifestAsset>(
			StaticLoadObject(
				UUsdPregenManifestAsset::StaticClass(),
				nullptr,
				*ObjectPath,
				nullptr,
				LOAD_NoWarn
			)
		);

		return Asset;
	}
}	 // namespace UE::UsdPregen