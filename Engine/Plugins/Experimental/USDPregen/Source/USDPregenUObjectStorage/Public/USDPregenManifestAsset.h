// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "USDPregenManifestAsset.generated.h"

namespace UE::UsdPregen
{
	class FManifest;
	class FTargetUid;
	struct FProduct;
}

/**
 * Reflected product entry for UObject-backed manifest storage.
 */
USTRUCT()
struct USDPREGENUOBJECTSTORAGE_API FUsdPregenProductEntry
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = "Products")
	FString UPackagePath;

	UPROPERTY(VisibleAnywhere, Category = "Products")
	FString UClass;

	UPROPERTY(VisibleAnywhere, Category = "Products")
	FString UNodeId;

	UPROPERTY(VisibleAnywhere, Category = "Products")
	FString UsdPrimType;

	UPROPERTY(VisibleAnywhere, Category = "Products")
	FString UsdPrimPath;

public:
	FUsdPregenProductEntry() = default;

	explicit FUsdPregenProductEntry(const UE::UsdPregen::FProduct& InProduct);

	UE::UsdPregen::FProduct ToWrapper() const;
};

/**
 * Reflected serialized PermutationOp argument (one entry under "opargs:").
 */
USTRUCT()
struct USDPREGENUOBJECTSTORAGE_API FUsdPregenSerializedOpArgEntry
{
	GENERATED_BODY()

public:
	/** Argument name with the "opargs:" prefix removed. */
	UPROPERTY(VisibleAnywhere, Category = "Op Arg")
	FString Name;

	/** Sdf value type name (e.g. "string"). */
	UPROPERTY(VisibleAnywhere, Category = "Op Arg")
	FString TypeName;

	/** String-encoded default value. */
	UPROPERTY(VisibleAnywhere, Category = "Op Arg")
	FString Value;
};

/**
 * Reflected serialized PermutationOp.
 */
USTRUCT()
struct USDPREGENUOBJECTSTORAGE_API FUsdPregenSerializedOpEntry
{
	GENERATED_BODY()

public:
	/** Op type name as authored by the op's Serialize() override. */
	UPROPERTY(VisibleAnywhere, Category = "Op")
	FString TypeName;

	UPROPERTY(VisibleAnywhere, Category = "Op")
	TArray<FUsdPregenSerializedOpArgEntry> Args;
};

/**
 * Reflected paired (definitionUid, permutationUid) tuple, used both for the
 * manifest's own target uid and for each entry in the dependencies array.
 */
USTRUCT()
struct USDPREGENUOBJECTSTORAGE_API FUsdPregenTargetUidEntry
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = "Target Uid")
	FString DefinitionUid;

	UPROPERTY(VisibleAnywhere, Category = "Target Uid")
	FString PermutationUid;
};

/**
 * Reflected ExtAssetDefinition snapshot, with the scene path and permutation
 * ops that the originating TargetDefinitionEntry carried for this definition folded
 * directly onto the definition entry.
 *
 * One entry per unique definition reachable from the originating TargetData
 * (deduped by UniqueId); the array is ordered by the asset definition stack
 * walk so the entries preserve namespace nesting order. Definitions are
 * re-registered into the AssetDefinitionRegistry on load.
 */
USTRUCT()
struct USDPREGENUOBJECTSTORAGE_API FUsdPregenDefinitionEntry
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = "Definition")
	FString UniqueId;

	UPROPERTY(VisibleAnywhere, Category = "Definition")
	FString Name;

	UPROPERTY(VisibleAnywhere, Category = "Definition")
	FString Version;

	UPROPERTY(VisibleAnywhere, Category = "Definition")
	FString IdentifierAuthored;

	UPROPERTY(VisibleAnywhere, Category = "Definition")
	FString IdentifierResolved;

	UPROPERTY(VisibleAnywhere, Category = "Definition")
	bool bHasCustomUniqueId = false;

	/**
	 * Round-tripped VtDictionary metadata. Stored as a USDA-format string blob;
	 * the encoding format is an implementation detail and may evolve.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Definition")
	FString Metadata;

	/** Scene path of the TargetDefinitionEntry that introduced this definition. */
	UPROPERTY(VisibleAnywhere, Category = "Definition")
	FString ScenePath;

	/**
	 * Permutation ops applied at this definition's scope of the target.
	 * Empty when this scope didn't contribute any ops.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Definition")
	TArray<FUsdPregenSerializedOpEntry> PermutationOps;
};

/**
 * UObject-backed manifest asset that can be serialized by Unreal and shown in the Content Browser.
 *
 * Note: only a subset of the originating TargetData is persisted on the
 * UAsset (target uid, dependencies, deduped definition snapshots with their
 * scene path and permutation ops). The full TargetData remains available to
 * downstream code via FManifest::GetTargetData() and can be persisted by a
 * custom storage plugin if richer state is needed.
 */
UCLASS(BlueprintType)
class USDPREGENUOBJECTSTORAGE_API UUsdPregenManifestAsset : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = "USD Pregen Manifest")
	TArray<FUsdPregenProductEntry> Products;

	UPROPERTY(VisibleAnywhere, Category = "USD Pregen Manifest|Target")
	FUsdPregenTargetUidEntry Uid;

	UPROPERTY(VisibleAnywhere, Category = "USD Pregen Manifest|Target")
	TArray<FUsdPregenTargetUidEntry> Dependencies;

	/** Deduped definition snapshots, keyed by UniqueId, ordered by namespace stack walk. */
	UPROPERTY(VisibleAnywhere, Category = "USD Pregen Manifest")
	TArray<FUsdPregenDefinitionEntry> Definitions;

public:
	UFUNCTION(BlueprintCallable, Category = "USD Pregen Manifest")
	bool IsValidManifest() const;

	void FromWrapper(const UE::UsdPregen::FManifest& InManifest);

	UE::UsdPregen::FManifest ToWrapper() const;
};
