// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Delegates/IDelegateInstance.h"

class FUSDPregenUObjectStorageModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:

	FDelegateHandle OnPostEngineInitHandle;
};
