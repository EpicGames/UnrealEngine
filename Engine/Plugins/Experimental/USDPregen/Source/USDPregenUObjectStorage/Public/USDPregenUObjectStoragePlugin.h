// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UsdPregenWrappers/IStoragePlugin.h"

class UUsdPregenManifestAsset;

namespace UE::UsdPregen
{
	struct FManifestPayload;
	struct FManifestLoadResult;
	struct FManifestSaveResult;

	class FExtAssetDefinition;
	class FManifest;
	class FTargetUid;

	// A storage plugin instance is expected to live for a single import. The
	// constructor captures the user's import content folder from
	// UUSDPregenSettings and uses it to place each manifest UAsset in the
	// same content folder as that target's products. Reusing one instance
	// across imports with different content paths would surface stale paths.
	class USDPREGENUOBJECTSTORAGE_API FUsdPregenUObjectStoragePlugin : public IStoragePlugin
	{
	public:
		FUsdPregenUObjectStoragePlugin();

		explicit FUsdPregenUObjectStoragePlugin(const FPregenStorageOptions& Options);

		virtual FManifestLoadResult LoadManifestPayload(const FTargetUid& TargetUid) override;

		virtual FManifestSaveResult StoreManifestPayload(
			const FTargetUid& TargetUid,
			const FManifestPayload& Payload
		) override;

		virtual FManifestSaveResult PersistManifestPayload(const FTargetUid& TargetUid) override;

		virtual FManifestPayload SerializeManifest(const FManifest& Manifest) override;

		virtual FManifest DeserializeManifestPayload(const FManifestPayload& Payload) override;

		virtual FString GetNameForUAsset(
			const FTargetUid& TargetUid,
			const TArray<FExtAssetDefinition>& Definitions,
			const FString& AssetType
		) override;

		virtual FString GetPackageSubPathForUAsset(
			const FTargetUid& TargetUid,
			const TArray<FExtAssetDefinition>& Definitions,
			const FString& AssetType
		) override;

		virtual FString GetPathForManifest(const FTargetUid& TargetUid) override;

	private:
		static FString GetPayloadEncoding();

		FString MakeManifestAssetName(const FTargetUid& TargetUid) const;
		FString MakeManifestObjectPath(const FTargetUid& TargetUid);

		UUsdPregenManifestAsset* LoadManifestAsset(const FTargetUid& TargetUid);

	private:
		FString ContentBasePath;

		// ${PLACEHOLDER} template used to build the package sub-path for
		// generated UAssets. See IStoragePlugin::ResolvePackageSubPathTemplate.
		// Falls back to UUSDPregenSettings::PackageSubPathTemplate, then to
		// IStoragePlugin::DefaultPackageSubPathTemplate().
		FString PackageSubPathTemplate;
	};
}	 // namespace UE::UsdPregen
