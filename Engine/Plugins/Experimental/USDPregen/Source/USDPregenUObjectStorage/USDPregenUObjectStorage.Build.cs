// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDPregenUObjectStorage : ModuleRules
	{
		public USDPregenUObjectStorage(ReadOnlyTargetRules Target) : base(Target)
		{            
			bUseRTTI = false;
			bLegalToDistributeObjectCode = true;
			bUseUnity = false;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealUSDWrapper",
					"USDPregenCore",
					"USDPregenInterchange",
					"USDPregenWrapper"
				}
			);
		}
	}
}