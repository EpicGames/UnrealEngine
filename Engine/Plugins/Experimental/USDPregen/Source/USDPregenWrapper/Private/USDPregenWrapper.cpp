// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenWrapper.h"

#include "USDLog.h"
#include "USDMemory.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdStage.h"

#include "UsdPregenWrappers/SceneTracker.h"

#define LOCTEXT_NAMESPACE "USDPregenWrapper"

#if USE_USD_SDK

#include "UsdPregen/sceneTracker.h"

#include "USDIncludesStart.h"
#if !USE_USD_MEMORY_MANAGER
#include "pxr/base/arch/memoryOverloads.h"
#endif // USE_USD_MEMORY_MANAGER
#include "pxr/base/tf/token.h"
#include "USDIncludesEnd.h"

using std::string;
using std::vector;

using namespace pxr;

PREGEN_NAMESPACE_USING_DIRECTIVE

#endif	  // USE_USD_SDK

namespace UE::USDPregenWrapper::Private
{
	UE::UsdPregen::FSceneTracker CreateSceneTrackerImpl(const UE::FUsdStage& Stage, const FPregenDiscoveryOptions& Options)
	{
#if USE_USD_SDK
		SceneTrackerRefPtr NativeSceneTracker = [&]() -> SceneTrackerRefPtr
		{
			FScopedUsdAllocs UsdAllocs;

			pxr::TfTokenVector Purposes;
			Purposes.reserve(Options.Purposes.Num());
			for (const FString& Purpose : Options.Purposes)
			{
				Purposes.emplace_back(TCHAR_TO_UTF8(*Purpose));
			}

			pxr::TfTokenVector ExcludeVariantSets;
			ExcludeVariantSets.reserve(Options.ExcludeVariantSets.Num());
			for (const FString& Entry : Options.ExcludeVariantSets)
			{
				ExcludeVariantSets.emplace_back(TCHAR_TO_UTF8(*Entry));
			}

			DiscoveryOptions DiscoveryOptions{
				static_cast<DiscoveryMode>(Options.DiscoveryMode),
				TCHAR_TO_UTF8(*Options.DiscoveryPluginName),
				TCHAR_TO_UTF8(*Options.DefinitionPrefix),
				UE::FSdfPath{ *Options.InitialPath },
				Purposes,
				ExcludeVariantSets,
				static_cast<IdentifierFallbackMode>(Options.AssetIdentifierFallback),
				static_cast<VersionFallbackMode>(Options.AssetVersionFallback)
			};

			return SceneTracker::Create(Stage, DiscoveryOptions);
		}();

		FScopedUnrealAllocs UnrealAllocs;
		return UE::UsdPregen::FSceneTracker(NativeSceneTracker);
#else
		return UE::UsdPregen::FSceneTracker();
#endif // #if USE_USD_SDK
	}
} // namespace UE::UsdPregenWrapper::Private

UE::UsdPregen::FSceneTracker USDPregenWrapper::CreateSceneTracker(
	const UE::FUsdStage& Stage)
{
	const FPregenDiscoveryOptions Options;
	return UE::USDPregenWrapper::Private::CreateSceneTrackerImpl(Stage, Options);
}

UE::UsdPregen::FSceneTracker USDPregenWrapper::CreateSceneTracker(
	const UE::FUsdStage& Stage,
	const FPregenDiscoveryOptions& Options)
{
	return UE::USDPregenWrapper::Private::CreateSceneTrackerImpl(Stage, Options);
}

#undef LOCTEXT_NAMESPACE
