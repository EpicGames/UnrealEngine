// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPregenWrapperModule.h"

#include "Modules/ModuleManager.h"

#include "USDMemory.h"

void FUSDPregenWrapperModule::StartupModule()
{
}

void FUSDPregenWrapperModule::ShutdownModule()
{
}

IMPLEMENT_MODULE_USD(FUSDPregenWrapperModule, USDPregenWrapper)