// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/AssetDefinitionRegistry.h"

#include "UsdPregenWrappers/ExtAssetDefinition.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "UsdPregen/assetDefinitionRegistry.h"
#include "UsdPregen/extAssetDefinition.h"
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{

#if USE_USD_SDK
	FAssetDefinitionRegistry::FAssetDefinitionRegistry(PREGEN_NS::AssetDefinitionRegistry* InRegistry)
		: PregenRegistry(InRegistry)
	{
	}

	FAssetDefinitionRegistry::operator const PREGEN_NS::AssetDefinitionRegistry* () const
	{
		return PregenRegistry;
	}
#endif	  // #if USE_USD_SDK

	// static
	FAssetDefinitionRegistry FAssetDefinitionRegistry::GetInstance()
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return FAssetDefinitionRegistry{ &PREGEN_NS::AssetDefinitionRegistry::GetInstance() };
#else
		return FAssetDefinitionRegistry{};
#endif	  // #if USE_USD_SDK
	}

	FExtAssetDefinition FAssetDefinitionRegistry::GetDefinition(const FString& UniqueId) const
	{
#if USE_USD_SDK
		if (PregenRegistry)
		{
			FScopedUsdAllocs UsdAllocs;

			if (const PREGEN_NS::ExtAssetDefinition* Definition =
				    PregenRegistry->GetDefinition(TCHAR_TO_UTF8(*UniqueId)))
			{
				return FExtAssetDefinition{ Definition };
			}
		}
#endif	  // #if USE_USD_SDK

		return FExtAssetDefinition{};
	}
}	 // namespace UE::UsdPregen