// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/ExtAssetDefinition.h"

#include "UnrealUSDWrapper.h" // for FSdfAssetPath
#include "UsdWrappers/VtValue.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "UsdPregen/assetDefinitionRegistry.h"
#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/util.h"

#include "USDIncludesStart.h"
#include "pxr/base/vt/dictionary.h"
#include "pxr/base/vt/value.h"
#include "pxr/usd/sdf/assetPath.h"
#include "USDIncludesEnd.h"

#include <optional>
#include <string>
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
#if USE_USD_SDK
	FExtAssetDefinition::FExtAssetDefinition(const PREGEN_NS::ExtAssetDefinition* InDefinition)
		: PregenDefinition(InDefinition)
	{
	}

	FExtAssetDefinition::operator const PREGEN_NS::ExtAssetDefinition* () const
	{
		return PregenDefinition;
	}
#endif	  // #if USE_USD_SDK

	bool FExtAssetDefinition::operator==(const FExtAssetDefinition& Other) const
	{
#if USE_USD_SDK
		if (PregenDefinition == nullptr || Other.PregenDefinition == nullptr)
		{
			return PregenDefinition == Other.PregenDefinition;
		}

		return *PregenDefinition == *Other.PregenDefinition;
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FExtAssetDefinition::operator!=(const FExtAssetDefinition& Other) const
	{
		return !(*this == Other);
	}

	bool FExtAssetDefinition::operator<(const FExtAssetDefinition& Other) const
	{
#if USE_USD_SDK
		if (PregenDefinition == nullptr || Other.PregenDefinition == nullptr)
		{
			return PregenDefinition < Other.PregenDefinition;
		}

		return *PregenDefinition < *Other.PregenDefinition;
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FExtAssetDefinition::operator bool() const
	{
#if USE_USD_SDK
		return PregenDefinition && static_cast<bool>(*PregenDefinition);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FString FExtAssetDefinition::GetName() const
	{
#if USE_USD_SDK
		if (PregenDefinition)
		{
			return UTF8_TO_TCHAR(PregenDefinition->GetName().c_str());
		}
#endif	  // #if USE_USD_SDK

		return FString();
	}

	FString FExtAssetDefinition::GetVersion() const
	{
#if USE_USD_SDK
		if (PregenDefinition)
		{
			return UTF8_TO_TCHAR(PregenDefinition->GetVersion().c_str());
		}
#endif	  // #if USE_USD_SDK

		return FString();
	}

	FSdfAssetPath FExtAssetDefinition::GetIdentifier() const
	{
#if USE_USD_SDK
		if (PregenDefinition)
		{
			const auto& Identifier = PregenDefinition->GetIdentifier();
			return FSdfAssetPath{
				UTF8_TO_TCHAR(Identifier.GetAuthoredPath().c_str()),
				UTF8_TO_TCHAR(Identifier.GetResolvedPath().c_str())
			};
		}
#endif	  // #if USE_USD_SDK

		return FSdfAssetPath();
	}

	FString FExtAssetDefinition::GetUniqueId() const
	{
#if USE_USD_SDK
		if (PregenDefinition)
		{
			return UTF8_TO_TCHAR(PregenDefinition->GetUniqueId().c_str());
		}
#endif	  // #if USE_USD_SDK

		return FString();
	}

	bool FExtAssetDefinition::GetMetadata(UE::FVtValue& OutMetadata) const
	{
#if USE_USD_SDK
		if (PregenDefinition)
		{
			if (const std::optional<pxr::VtDictionary>& Metadata = PregenDefinition->GetMetadata())
			{
				OutMetadata = UE::FVtValue{ pxr::VtValue{ *Metadata } };
				return true;
			}
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FExtAssetDefinition::HasMetadata() const
	{
#if USE_USD_SDK
		if (PregenDefinition)
		{
			return PregenDefinition->HasMetadata();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FExtAssetDefinition::HasCustomUniqueId() const
	{
#if USE_USD_SDK
		if (PregenDefinition)
		{
			return PregenDefinition->HasCustomUniqueId();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FExtAssetDefinition::IsValid(FString* OutReason) const
	{
#if USE_USD_SDK
		if (PregenDefinition)
		{
			FScopedUsdAllocs UsdAllocs;

			std::string Reason;
			const bool bValid = PregenDefinition->IsValid(OutReason ? &Reason : nullptr);

			if (OutReason)
			{
				*OutReason = UTF8_TO_TCHAR(Reason.c_str());
			}

			return bValid;
		}

		if (OutReason)
		{
			OutReason->Reset();
		}
		return false;
#else
		if (OutReason)
		{
			OutReason->Reset();
		}
		return false;
#endif	  // #if USE_USD_SDK
	}

	FExtAssetDefinitionSnapshot
	FExtAssetDefinitionSnapshot::From(const FExtAssetDefinition& Defn)
	{
		FExtAssetDefinitionSnapshot Snapshot;

#if USE_USD_SDK
		const PREGEN_NS::ExtAssetDefinition* PregenDefn = Defn;
		if (!PregenDefn)
		{
			return Snapshot;
		}

		FScopedUsdAllocs UsdAllocs;

		Snapshot.UniqueId = UTF8_TO_TCHAR(PregenDefn->GetUniqueId().c_str());
		Snapshot.Name = UTF8_TO_TCHAR(PregenDefn->GetName().c_str());
		Snapshot.Version = UTF8_TO_TCHAR(PregenDefn->GetVersion().c_str());

		const pxr::SdfAssetPath& Identifier = PregenDefn->GetIdentifier();
		Snapshot.IdentifierAuthored = UTF8_TO_TCHAR(Identifier.GetAuthoredPath().c_str());
		Snapshot.IdentifierResolved = UTF8_TO_TCHAR(Identifier.GetResolvedPath().c_str());

		Snapshot.bHasCustomUniqueId = PregenDefn->HasCustomUniqueId();

		if (const std::optional<pxr::VtDictionary>& Metadata = PregenDefn->GetMetadata())
		{
			const std::string Blob
				= PREGEN_NS::internal::util::MetadataToUsdaBlob(*Metadata);
			Snapshot.Metadata = UTF8_TO_TCHAR(Blob.c_str());
		}
#endif	  // #if USE_USD_SDK

		return Snapshot;
	}

	bool FExtAssetDefinitionSnapshot::RegisterIntoRegistry() const
	{
#if USE_USD_SDK
		if (UniqueId.IsEmpty())
		{
			return false;
		}

		FScopedUsdAllocs UsdAllocs;

		const std::string NativeUniqueId = TCHAR_TO_UTF8(*UniqueId);
		const std::string NativeName = TCHAR_TO_UTF8(*Name);
		const std::string NativeVersion = TCHAR_TO_UTF8(*Version);
		const std::string NativeAuthored = TCHAR_TO_UTF8(*IdentifierAuthored);
		const std::string NativeResolved = TCHAR_TO_UTF8(*IdentifierResolved);
		const std::string NativeMetadataBlob = TCHAR_TO_UTF8(*Metadata);

		const pxr::SdfAssetPath Identifier{NativeAuthored, NativeResolved};

		// Pick the constructor variant that preserves HasCustomUniqueId()
		// across the round-trip. When the source definition had a custom
		// UID, the customUniqueId form must be used so the value is stored
		// verbatim. When the source UID was default-derived from name +
		// version, the no-customUniqueId form lets ExtAssetDefinition
		// regenerate the same UID and report HasCustomUniqueId() == false.
		const std::optional<pxr::VtDictionary> DecodedMetadata
			= PREGEN_NS::internal::util::UsdaBlobToMetadata(NativeMetadataBlob);

		std::optional<PREGEN_NS::ExtAssetDefinition> Defn;
		if (bHasCustomUniqueId)
		{
			if (DecodedMetadata)
			{
				Defn.emplace(NativeName, NativeVersion, Identifier,
				             NativeUniqueId, *DecodedMetadata);
			}
			else
			{
				Defn.emplace(NativeName, NativeVersion, Identifier, NativeUniqueId);
			}
		}
		else
		{
			if (DecodedMetadata)
			{
				Defn.emplace(NativeName, NativeVersion, Identifier, *DecodedMetadata);
			}
			else
			{
				Defn.emplace(NativeName, NativeVersion, Identifier);
			}
		}

		// AddDefinition logs a detailed conflict warning internally when the
		// UID exists with different fields, so we don't double-warn here.
		PREGEN_NS::AssetDefinitionRegistry& Registry
			= PREGEN_NS::AssetDefinitionRegistry::GetInstance();
		return Registry.AddDefinition(*Defn) != nullptr;
#else
		return false;
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE::UsdPregen