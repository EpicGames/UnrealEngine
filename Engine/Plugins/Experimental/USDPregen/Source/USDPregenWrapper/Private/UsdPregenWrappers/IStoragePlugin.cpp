// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/IStoragePlugin.h"

#include "UsdPregenWrappers/ExtAssetDefinition.h"
#include "UsdPregenWrappers/ManifestTypes.h"
#include "UsdPregenWrappers/Target.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/storagePlugin.h"

#include <string>
#include <unordered_map>
#include <vector>
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	IStoragePlugin::~IStoragePlugin() = default;

	FManifestSaveResult IStoragePlugin::PersistManifestPayload(const FTargetUid& TargetUid)
	{
		return {};
	}

	// static
	const FString& IStoragePlugin::DefaultPackageSubPathTemplate()
	{
#if USE_USD_SDK
		static const FString sTemplate = [&]()
			{
				FScopedUsdAllocs UsdAllocs;
				return FString{ UTF8_TO_TCHAR(PREGEN_NS::StoragePlugin::DefaultPackageSubPathTemplate().c_str()) };
			}();
		return sTemplate;
#else
		static const FString sTemplate;
		return sTemplate;
#endif	  // #if USE_USD_SDK
	}

	// static
	const FString& IStoragePlugin::EmptyValueSentinel()
	{
#if USE_USD_SDK
		static const FString sSentinel = [&]()
			{
				FScopedUsdAllocs UsdAllocs;
				return FString{ UTF8_TO_TCHAR(PREGEN_NS::StoragePlugin::EmptyValueSentinel().c_str()) };
			}();
		return sSentinel;
#else
		static const FString sSentinel{ TEXT("_") };
		return sSentinel;
#endif	  // #if USE_USD_SDK
	}

	// static
	FString IStoragePlugin::ResolvePackageSubPathTemplate(
		const FString& Template,
		const FTargetUid& TargetUid,
		const TArray<FExtAssetDefinition>& Definitions,
		const FString& AssetType,
		const TMap<FString, FString>& ExtraSubstitutions
	)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		// Bridge the Unreal-side inputs into native std types and hand
		// off to the core implementation. Keeping the helper in one
		// place (instead of duplicating the parser in Unreal-land)
		// guarantees the substitution semantics stay consistent across
		// every storage plugin.
		std::vector<const PREGEN_NS::ExtAssetDefinition*> NativeDefinitions;
		NativeDefinitions.reserve(Definitions.Num());
		for (const FExtAssetDefinition& Definition : Definitions)
		{
			if (const PREGEN_NS::ExtAssetDefinition* NativeDefinition = Definition)
			{
				NativeDefinitions.push_back(NativeDefinition);
			}
		}

		std::unordered_map<std::string, std::string> NativeExtra;
		NativeExtra.reserve(static_cast<std::size_t>(ExtraSubstitutions.Num()));
		for (const TPair<FString, FString>& Pair : ExtraSubstitutions)
		{
			NativeExtra.emplace(TCHAR_TO_UTF8(*Pair.Key), TCHAR_TO_UTF8(*Pair.Value));
		}

		const std::string NativeResult = PREGEN_NS::StoragePlugin::ResolvePackageSubPathTemplate(
			TCHAR_TO_UTF8(*Template),
			TargetUid,
			NativeDefinitions,
			TCHAR_TO_UTF8(*AssetType),
			NativeExtra
		);

		FScopedUnrealAllocs UnrealAllocs;
		return FString{ UTF8_TO_TCHAR(NativeResult.c_str()) };
#else
		return FString{};
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE::UsdPregen
