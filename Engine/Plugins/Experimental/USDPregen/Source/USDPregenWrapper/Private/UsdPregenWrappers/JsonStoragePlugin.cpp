// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/JsonStoragePlugin.h"

#include "UsdPregenWrappers/ExtAssetDefinition.h"
#include "UsdPregenWrappers/Manifest.h"
#include "UsdPregenWrappers/ManifestTypes.h"
#include "UsdPregenWrappers/Target.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "UsdPregen/jsonStoragePlugin.h"
#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/manifest.h"
#include "UsdPregen/manifestTypes.h"
#include "UsdPregen/storageOptions.h"
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	namespace Internal
	{
		template<typename PtrType>
		class FJsonStoragePluginImpl
		{
		public:
			FJsonStoragePluginImpl() = default;

			explicit FJsonStoragePluginImpl(const PtrType& InPtr)
				: PregenPtr(InPtr)
			{
			}

			explicit FJsonStoragePluginImpl(PtrType&& InPtr)
				: PregenPtr(MoveTemp(InPtr))
			{
			}

			PtrType& GetInner()
			{
				return PregenPtr.Get();
			}

			const PtrType& GetInner() const
			{
				return PregenPtr.Get();
			}

		private:
			TUsdStore<PtrType> PregenPtr;
		};
	}	 // namespace Internal

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::FJsonStoragePluginBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FJsonStoragePluginImpl<PtrType>>();
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::FJsonStoragePluginBase(const FPregenStorageOptions& Options)
	{
#if USE_USD_SDK
		PtrType NativePlugin;
		{
			FScopedUsdAllocs UsdAllocs;
			PREGEN_NS::StorageOptions NativeOptions;
			NativeOptions.manifestDir = TCHAR_TO_UTF8(*Options.ManifestDir);
			NativeOptions.packageSubPathTemplate = TCHAR_TO_UTF8(*Options.PackageSubPathTemplate);
			NativePlugin = PtrType{ new PREGEN_NS::JsonStoragePlugin{ NativeOptions } };
		}

		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FJsonStoragePluginImpl<PtrType>>(MoveTemp(NativePlugin));
#else
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FJsonStoragePluginImpl<PtrType>>();
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::FJsonStoragePluginBase(const FJsonStoragePlugin& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FJsonStoragePluginImpl<PtrType>>(Other.Impl->GetInner());
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::FJsonStoragePluginBase(FJsonStoragePlugin&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FJsonStoragePluginImpl<PtrType>>(MoveTemp(Other.Impl->GetInner()));
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>& FJsonStoragePluginBase<PtrType>::operator=(const FJsonStoragePlugin& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FJsonStoragePluginImpl<PtrType>>(Other.Impl->GetInner());
		return *this;
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>& FJsonStoragePluginBase<PtrType>::operator=(FJsonStoragePlugin&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FJsonStoragePluginImpl<PtrType>>(MoveTemp(Other.Impl->GetInner()));
		return *this;
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::~FJsonStoragePluginBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Impl->GetInner());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::FJsonStoragePluginBase(const PREGEN_NS::JsonStoragePluginRefPtr& InPtr)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FJsonStoragePluginImpl<PtrType>>(PtrType{ InPtr });
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::FJsonStoragePluginBase(PREGEN_NS::JsonStoragePluginRefPtr&& InPtr)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FJsonStoragePluginImpl<PtrType>>(PtrType{ MoveTemp(InPtr) });
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::operator PtrType& ()
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::operator const PtrType& () const
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FJsonStoragePluginBase<PtrType>::operator PREGEN_NS::JsonStoragePluginRefPtr() const
	{
		return Impl->GetInner();
	}
#endif	  // #if USE_USD_SDK

	template<typename PtrType>
	FManifestLoadResult FJsonStoragePluginBase<PtrType>::LoadManifestPayload(const FTargetUid& TargetUid) const
	{
#if USE_USD_SDK
		if (Impl->GetInner())
		{
			auto NativeResult = [&]()
				{
					FScopedUsdAllocs UsdAllocs;
					return Impl->GetInner()->LoadManifestPayload(TargetUid);
				}();

			return FManifestLoadResult{ NativeResult };
		}
#endif	  // #if USE_USD_SDK

		return FManifestLoadResult{};
	}

	template<typename PtrType>
	FManifestSaveResult FJsonStoragePluginBase<PtrType>::StoreManifestPayload(
		const FTargetUid& TargetUid,
		const FManifestPayload& Payload
	) const
	{
#if USE_USD_SDK
		if (Impl->GetInner())
		{
			auto NativeResult = [&]()
				{
					FScopedUsdAllocs UsdAllocs;
					return Impl->GetInner()->StoreManifestPayload(TargetUid, Payload);
				}();

			return FManifestSaveResult{ NativeResult };
		}
#endif	  // #if USE_USD_SDK

		return FManifestSaveResult{};
	}

	template<typename PtrType>
	FManifestSaveResult FJsonStoragePluginBase<PtrType>::PersistManifestPayload(const FTargetUid& TargetUid) const
	{
#if USE_USD_SDK
		if (Impl->GetInner())
		{
			auto NativeResult = [&]()
				{
					FScopedUsdAllocs UsdAllocs;
					return Impl->GetInner()->PersistManifestPayload(TargetUid);
				}();

			return FManifestSaveResult{ NativeResult };
		}
#endif	  // #if USE_USD_SDK

		return FManifestSaveResult{};
	}

	template<typename PtrType>
	FManifestPayload FJsonStoragePluginBase<PtrType>::SerializeManifest(const FManifest& Manifest) const
	{
#if USE_USD_SDK
		if (Impl->GetInner())
		{
			auto NativeResult = [&]()
				{
					FScopedUsdAllocs UsdAllocs;
					return Impl->GetInner()->SerializeManifest(Manifest);
				}();

			return FManifestPayload{ NativeResult };
		}
#endif	  // #if USE_USD_SDK

		return FManifestPayload{};
	}

	template<typename PtrType>
	FManifest FJsonStoragePluginBase<PtrType>::DeserializeManifestPayload(const FManifestPayload& Payload) const
	{
#if USE_USD_SDK
		if (Impl->GetInner())
		{
			auto NativeResult = [&]()
				{
					FScopedUsdAllocs UsdAllocs;
					return Impl->GetInner()->DeserializeManifestPayload(Payload);
				}();
			return FManifest{ NativeResult };
		}
#endif	  // #if USE_USD_SDK

		return FManifest{};
	}

	template<typename PtrType>
	FString FJsonStoragePluginBase<PtrType>::GetNameForUAsset(
		const FTargetUid& TargetUid,
		const TArray<FExtAssetDefinition>& Definitions,
		const FString& AssetType
	) const
	{
#if USE_USD_SDK
		if (Impl->GetInner())
		{
			auto NativeResult = [&]()
				{
					FScopedUsdAllocs UsdAllocs;

					std::vector<const PREGEN_NS::ExtAssetDefinition*> NativeDefinitions;
					NativeDefinitions.reserve(Definitions.Num());

					for (const FExtAssetDefinition& Definition : Definitions)
					{
						if (const PREGEN_NS::ExtAssetDefinition* NativeDefinition = Definition)
						{
							NativeDefinitions.push_back(NativeDefinition);
						}
					}

					return Impl->GetInner()->GetNameForUAsset(
						TargetUid,
						NativeDefinitions,
						TCHAR_TO_UTF8(*AssetType));
				}();

			FScopedUnrealAllocs UnrealAllocs;
			return UTF8_TO_TCHAR(NativeResult.c_str());
		}
#endif	  // #if USE_USD_SDK

		return FString{};
	}

	template<typename PtrType>
	FString FJsonStoragePluginBase<PtrType>::GetPackageSubPathForUAsset(
		const FTargetUid& TargetUid,
		const TArray<FExtAssetDefinition>& Definitions,
		const FString& AssetType
	) const
	{
#if USE_USD_SDK
		if (Impl->GetInner())
		{
			auto NativeResult = [&]()
				{
					FScopedUsdAllocs UsdAllocs;

					std::vector<const PREGEN_NS::ExtAssetDefinition*> NativeDefinitions;
					NativeDefinitions.reserve(Definitions.Num());

					for (const FExtAssetDefinition& Definition : Definitions)
					{
						if (const PREGEN_NS::ExtAssetDefinition* NativeDefinition = Definition)
						{
							NativeDefinitions.push_back(NativeDefinition);
						}
					}

					return Impl->GetInner()->GetPackageSubPathForUAsset(
						TargetUid,
						NativeDefinitions,
						TCHAR_TO_UTF8(*AssetType));
				}();

			FScopedUnrealAllocs UnrealAllocs;
			return UTF8_TO_TCHAR(NativeResult.c_str());
		}
#endif	  // #if USE_USD_SDK

		return FString{};
	}

#if USE_USD_SDK
	template class FJsonStoragePluginBase<PREGEN_NS::JsonStoragePluginRefPtr>;
#else
	template class FJsonStoragePluginBase<FDummyRefPtrType>;
#endif	  // #if USE_USD_SDK
}	 // namespace UE::UsdPregen