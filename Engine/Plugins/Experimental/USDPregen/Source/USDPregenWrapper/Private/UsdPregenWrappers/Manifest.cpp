// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/Manifest.h"

#include "UsdPregen/pregen.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "UsdPregen/manifest.h"
#include "UsdPregen/target.h"
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	namespace Internal
	{
		class FManifestImpl
		{
		public:
			FManifestImpl() = default;

#if USE_USD_SDK
			explicit FManifestImpl(const PREGEN_NS::Manifest& InManifest)
				: PregenManifest(InManifest)
			{
			}

			explicit FManifestImpl(PREGEN_NS::Manifest&& InManifest)
				: PregenManifest(MoveTemp(InManifest))
			{
			}

			TUsdStore<PREGEN_NS::Manifest> PregenManifest;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal

	FProduct::FProduct() = default;

	FProduct::FProduct(const FProduct& Other) = default;
	FProduct::FProduct(FProduct&& Other) = default;

	FProduct& FProduct::operator=(const FProduct& Other) = default;
	FProduct& FProduct::operator=(FProduct&& Other) = default;

#if USE_USD_SDK
	FProduct::FProduct(const PREGEN_NS::Product& InProduct)
		: UPackagePath(UTF8_TO_TCHAR(InProduct.upackagePath.c_str()))
		, UClass(UTF8_TO_TCHAR(InProduct.uclass.c_str()))
		, UNodeId(UTF8_TO_TCHAR(InProduct.unodeId.c_str()))
		, UsdPrimType(UTF8_TO_TCHAR(InProduct.usdPrimType.c_str()))
		, UsdPrimPath(UTF8_TO_TCHAR(InProduct.usdPrimPath.c_str()))
	{
	}

	FProduct::FProduct(PREGEN_NS::Product&& InProduct)
		: UPackagePath(UTF8_TO_TCHAR(InProduct.upackagePath.c_str()))
		, UClass(UTF8_TO_TCHAR(InProduct.uclass.c_str()))
		, UNodeId(UTF8_TO_TCHAR(InProduct.unodeId.c_str()))
		, UsdPrimType(UTF8_TO_TCHAR(InProduct.usdPrimType.c_str()))
		, UsdPrimPath(UTF8_TO_TCHAR(InProduct.usdPrimPath.c_str()))
	{
	}

	FProduct& FProduct::operator=(const PREGEN_NS::Product& InProduct)
	{
		UPackagePath = UTF8_TO_TCHAR(InProduct.upackagePath.c_str());
		UClass = UTF8_TO_TCHAR(InProduct.uclass.c_str());
		UNodeId = UTF8_TO_TCHAR(InProduct.unodeId.c_str());
		UsdPrimType = UTF8_TO_TCHAR(InProduct.usdPrimType.c_str());
		UsdPrimPath = UTF8_TO_TCHAR(InProduct.usdPrimPath.c_str());

		return *this;
	}

	FProduct& FProduct::operator=(PREGEN_NS::Product&& InProduct)
	{
		UPackagePath = UTF8_TO_TCHAR(InProduct.upackagePath.c_str());
		UClass = UTF8_TO_TCHAR(InProduct.uclass.c_str());
		UNodeId = UTF8_TO_TCHAR(InProduct.unodeId.c_str());
		UsdPrimType = UTF8_TO_TCHAR(InProduct.usdPrimType.c_str());
		UsdPrimPath = UTF8_TO_TCHAR(InProduct.usdPrimPath.c_str());

		return *this;
	}

	FProduct::operator PREGEN_NS::Product() const
	{
		FScopedUsdAllocs UsdAllocs;

		PREGEN_NS::Product Result;
		Result.upackagePath = TCHAR_TO_UTF8(*UPackagePath);
		Result.uclass = TCHAR_TO_UTF8(*UClass);
		Result.unodeId = TCHAR_TO_UTF8(*UNodeId);
		Result.usdPrimType = TCHAR_TO_UTF8(*UsdPrimType);
		Result.usdPrimPath = TCHAR_TO_UTF8(*UsdPrimPath);

		return Result;
	}
#endif	  // #if USE_USD_SDK

	FManifest::FManifest()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FManifestImpl>();
	}

	FManifest::FManifest(const FManifest& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FManifestImpl>(Other.Impl->PregenManifest.Get());
#else
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FManifestImpl>();
#endif	  // #if USE_USD_SDK
	}

	FManifest::FManifest(FManifest&& Other) = default;

	FManifest::~FManifest()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FManifest& FManifest::operator=(const FManifest& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FManifestImpl>(Other.Impl->PregenManifest.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FManifest& FManifest::operator=(FManifest&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);
		return *this;
	}

	FManifest::operator bool() const
	{
#if USE_USD_SDK
		return Impl->PregenManifest.Get().IsValid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FManifest::FManifest(const PREGEN_NS::Manifest& InManifest)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FManifestImpl>(InManifest);
	}

	FManifest::FManifest(PREGEN_NS::Manifest&& InManifest)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FManifestImpl>(MoveTemp(InManifest));
	}

	FManifest& FManifest::operator=(const PREGEN_NS::Manifest& InManifest)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FManifestImpl>(InManifest);
		return *this;
	}

	FManifest& FManifest::operator=(PREGEN_NS::Manifest&& InManifest)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FManifestImpl>(MoveTemp(InManifest));
		return *this;
	}

	FManifest::operator PREGEN_NS::Manifest& ()
	{
		return Impl->PregenManifest.Get();
	}

	FManifest::operator const PREGEN_NS::Manifest& () const
	{
		return Impl->PregenManifest.Get();
	}
#endif	  // #if USE_USD_SDK

	FTargetUid FManifest::GetTargetUid() const
	{
#if USE_USD_SDK
		return FTargetUid{ Impl->PregenManifest.Get().GetTargetUid() };
#else
		return FTargetUid{};
#endif	  // #if USE_USD_SDK
	}

	void FManifest::AddProduct(const FProduct& Product)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		Impl->PregenManifest.Get().AddProduct(Product);
#endif	  // #if USE_USD_SDK
	}

	TArray<FProduct> FManifest::GetProducts() const
	{
		TArray<FProduct> Result;

#if USE_USD_SDK
		const std::vector<PREGEN_NS::Product>& Products = Impl->PregenManifest.Get().GetProducts();
		Result.Reserve(Products.size());

		for (const PREGEN_NS::Product& Product : Products)
		{
			Result.Add(FProduct{ Product });
		}
#endif	  // #if USE_USD_SDK

		return Result;
	}

	void FManifest::SetTargetData(const FTargetData& TargetData)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		Impl->PregenManifest.Get().SetTargetData(
			static_cast<PREGEN_NS::TargetDataRefPtr>(TargetData)
		);
#endif	  // #if USE_USD_SDK
	}

	FTargetData FManifest::GetTargetData() const
	{
#if USE_USD_SDK
		return FTargetData{ Impl->PregenManifest.Get().GetTargetData() };
#else
		return FTargetData{};
#endif	  // #if USE_USD_SDK
	}

	bool FManifest::IsValid() const
	{
#if USE_USD_SDK
		return Impl->PregenManifest.Get().IsValid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE::UsdPregen