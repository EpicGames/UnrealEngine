// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/ManifestTypes.h"

#include "UsdPregen/pregen.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "UsdPregen/manifestTypes.h"
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	namespace Internal
	{
#if USE_USD_SDK
		static EManifestLoadStatus ConvertManifestLoadStatus(PREGEN_NS::ManifestLoadStatus InStatus)
		{
			switch (InStatus)
			{
			case PREGEN_NS::ManifestLoadStatus::Loaded:
			{
				return EManifestLoadStatus::Loaded;
			}
			case PREGEN_NS::ManifestLoadStatus::DoesNotExist:
			{
				return EManifestLoadStatus::DoesNotExist;
			}
			case PREGEN_NS::ManifestLoadStatus::Error:
			default:
			{
				return EManifestLoadStatus::Error;
			}
			}
		}

		static PREGEN_NS::ManifestLoadStatus ConvertManifestLoadStatus(EManifestLoadStatus InStatus)
		{
			switch (InStatus)
			{
			case EManifestLoadStatus::Loaded:
			{
				return PREGEN_NS::ManifestLoadStatus::Loaded;
			}
			case EManifestLoadStatus::DoesNotExist:
			{
				return PREGEN_NS::ManifestLoadStatus::DoesNotExist;
			}
			case EManifestLoadStatus::Error:
			default:
			{
				return PREGEN_NS::ManifestLoadStatus::Error;
			}
			}
		}

		static EManifestSaveStatus ConvertManifestSaveStatus(PREGEN_NS::ManifestSaveStatus InStatus)
		{
			switch (InStatus)
			{
			case PREGEN_NS::ManifestSaveStatus::Saved:
			{
				return EManifestSaveStatus::Saved;
			}
			case PREGEN_NS::ManifestSaveStatus::NotSaved:
			{
				return EManifestSaveStatus::NotSaved;
			}
			case PREGEN_NS::ManifestSaveStatus::Error:
			default:
			{
				return EManifestSaveStatus::Error;
			}
			}
		}

		static PREGEN_NS::ManifestSaveStatus ConvertManifestSaveStatus(EManifestSaveStatus InStatus)
		{
			switch (InStatus)
			{
			case EManifestSaveStatus::Saved:
			{
				return PREGEN_NS::ManifestSaveStatus::Saved;
			}
			case EManifestSaveStatus::NotSaved:
			{
				return PREGEN_NS::ManifestSaveStatus::NotSaved;
			}
			case EManifestSaveStatus::Error:
			default:
			{
				return PREGEN_NS::ManifestSaveStatus::Error;
			}
			}
		}
#endif	  // #if USE_USD_SDK
	}	 // namespace Internal

	FManifestPayload::FManifestPayload() = default;
	FManifestPayload::FManifestPayload(const FManifestPayload& Other) = default;
	FManifestPayload::FManifestPayload(FManifestPayload&& Other) = default;
	FManifestPayload& FManifestPayload::operator=(const FManifestPayload& Other) = default;
	FManifestPayload& FManifestPayload::operator=(FManifestPayload&& Other) = default;

#if USE_USD_SDK
	FManifestPayload::FManifestPayload(const PREGEN_NS::ManifestPayload& InManifestPayload)
		: Encoding(UTF8_TO_TCHAR(InManifestPayload.encoding.c_str()))
	{
		FScopedUnrealAllocs UnrealAllocs;

		Data.Reserve(InManifestPayload.data.size());

		for (std::uint8_t Byte : InManifestPayload.data)
		{
			Data.Add(static_cast<uint8>(Byte));
		}
	}

	FManifestPayload::FManifestPayload(PREGEN_NS::ManifestPayload&& InManifestPayload)
		: Encoding(UTF8_TO_TCHAR(InManifestPayload.encoding.c_str()))
	{
		FScopedUnrealAllocs UnrealAllocs;

		Data.Reserve(InManifestPayload.data.size());

		for (std::uint8_t Byte : InManifestPayload.data)
		{
			Data.Add(static_cast<uint8>(Byte));
		}
	}

	FManifestPayload& FManifestPayload::operator=(const PREGEN_NS::ManifestPayload& InManifestPayload)
	{
		FScopedUnrealAllocs UnrealAllocs;

		Encoding = UTF8_TO_TCHAR(InManifestPayload.encoding.c_str());

		Data.Reset();
		Data.Reserve(InManifestPayload.data.size());

		for (std::uint8_t Byte : InManifestPayload.data)
		{
			Data.Add(static_cast<uint8>(Byte));
		}

		return *this;
	}

	FManifestPayload& FManifestPayload::operator=(PREGEN_NS::ManifestPayload&& InManifestPayload)
	{
		FScopedUnrealAllocs UnrealAllocs;

		Encoding = UTF8_TO_TCHAR(InManifestPayload.encoding.c_str());

		Data.Reset();
		Data.Reserve(InManifestPayload.data.size());

		for (std::uint8_t Byte : InManifestPayload.data)
		{
			Data.Add(static_cast<uint8>(Byte));
		}

		return *this;
	}

	FManifestPayload::operator PREGEN_NS::ManifestPayload() const
	{
		FScopedUsdAllocs UsdAllocs;

		PREGEN_NS::ManifestPayload Result;
		Result.encoding = TCHAR_TO_UTF8(*Encoding);
		Result.data.reserve(Data.Num());

		for (uint8 Byte : Data)
		{
			Result.data.push_back(static_cast<std::uint8_t>(Byte));
		}

		return Result;
	}
#endif	  // #if USE_USD_SDK

	FManifestLoadResult::FManifestLoadResult() = default;
	FManifestLoadResult::FManifestLoadResult(const FManifestLoadResult& Other) = default;
	FManifestLoadResult::FManifestLoadResult(FManifestLoadResult&& Other) = default;
	FManifestLoadResult& FManifestLoadResult::operator=(const FManifestLoadResult& Other) = default;
	FManifestLoadResult& FManifestLoadResult::operator=(FManifestLoadResult&& Other) = default;

#if USE_USD_SDK
	FManifestLoadResult::FManifestLoadResult(const PREGEN_NS::ManifestLoadResult& InManifestLoadResult)
		: Status(Internal::ConvertManifestLoadStatus(InManifestLoadResult.status))
		, Payload(InManifestLoadResult.payload)
		, Message(UTF8_TO_TCHAR(InManifestLoadResult.message.c_str()))
	{
	}

	FManifestLoadResult::FManifestLoadResult(PREGEN_NS::ManifestLoadResult&& InManifestLoadResult)
		: Status(Internal::ConvertManifestLoadStatus(InManifestLoadResult.status))
		, Payload(MoveTemp(InManifestLoadResult.payload))
		, Message(UTF8_TO_TCHAR(InManifestLoadResult.message.c_str()))
	{
	}

	FManifestLoadResult& FManifestLoadResult::operator=(const PREGEN_NS::ManifestLoadResult& InManifestLoadResult)
	{
		Status = Internal::ConvertManifestLoadStatus(InManifestLoadResult.status);
		Payload = FManifestPayload{ InManifestLoadResult.payload };
		Message = UTF8_TO_TCHAR(InManifestLoadResult.message.c_str());

		return *this;
	}

	FManifestLoadResult& FManifestLoadResult::operator=(PREGEN_NS::ManifestLoadResult&& InManifestLoadResult)
	{
		Status = Internal::ConvertManifestLoadStatus(InManifestLoadResult.status);
		Payload = FManifestPayload{ MoveTemp(InManifestLoadResult.payload) };
		Message = UTF8_TO_TCHAR(InManifestLoadResult.message.c_str());

		return *this;
	}

	FManifestLoadResult::operator PREGEN_NS::ManifestLoadResult() const
	{
		FScopedUsdAllocs UsdAllocs;

		PREGEN_NS::ManifestLoadResult Result;
		Result.status = Internal::ConvertManifestLoadStatus(Status);
		Result.payload = Payload;
		Result.message = TCHAR_TO_UTF8(*Message);

		return Result;
	}
#endif	  // #if USE_USD_SDK

	FManifestSaveResult::FManifestSaveResult() = default;
	FManifestSaveResult::FManifestSaveResult(const FManifestSaveResult& Other) = default;
	FManifestSaveResult::FManifestSaveResult(FManifestSaveResult&& Other) = default;
	FManifestSaveResult& FManifestSaveResult::operator=(const FManifestSaveResult& Other) = default;
	FManifestSaveResult& FManifestSaveResult::operator=(FManifestSaveResult&& Other) = default;

#if USE_USD_SDK
	FManifestSaveResult::FManifestSaveResult(const PREGEN_NS::ManifestSaveResult& InManifestSaveResult)
		: Status(Internal::ConvertManifestSaveStatus(InManifestSaveResult.status))
		, Message(UTF8_TO_TCHAR(InManifestSaveResult.message.c_str()))
	{
	}

	FManifestSaveResult::FManifestSaveResult(PREGEN_NS::ManifestSaveResult&& InManifestSaveResult)
		: Status(Internal::ConvertManifestSaveStatus(InManifestSaveResult.status))
		, Message(UTF8_TO_TCHAR(InManifestSaveResult.message.c_str()))
	{
	}

	FManifestSaveResult& FManifestSaveResult::operator=(const PREGEN_NS::ManifestSaveResult& InManifestSaveResult)
	{
		Status = Internal::ConvertManifestSaveStatus(InManifestSaveResult.status);
		Message = UTF8_TO_TCHAR(InManifestSaveResult.message.c_str());

		return *this;
	}

	FManifestSaveResult& FManifestSaveResult::operator=(PREGEN_NS::ManifestSaveResult&& InManifestSaveResult)
	{
		Status = Internal::ConvertManifestSaveStatus(InManifestSaveResult.status);
		Message = UTF8_TO_TCHAR(InManifestSaveResult.message.c_str());

		return *this;
	}

	FManifestSaveResult::operator PREGEN_NS::ManifestSaveResult() const
	{
		FScopedUsdAllocs UsdAllocs;

		PREGEN_NS::ManifestSaveResult Result;
		Result.status = Internal::ConvertManifestSaveStatus(Status);
		Result.message = TCHAR_TO_UTF8(*Message);

		return Result;
	}
#endif	  // #if USE_USD_SDK
}	 // namespace UE::UsdPregen