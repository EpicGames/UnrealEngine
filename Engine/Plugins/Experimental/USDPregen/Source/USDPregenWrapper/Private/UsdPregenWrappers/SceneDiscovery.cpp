// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/SceneDiscovery.h"

#include "UsdPregen/pregen.h"

#include "USDPregenWrapper.h"
#include "UsdPregenWrappers/Target.h"
#include "UsdPregenWrappers/SceneTracker.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdStage.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "UsdPregen/discoveryOptions.h"
#include "UsdPregen/target.h"
#include "UsdPregen/sceneTracker.h"
#include "UsdPregen/sceneDiscovery.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usd/stage.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	namespace Internal
	{
		class FSceneDiscoveryImpl
		{
		public:
#if USE_USD_SDK
			explicit FSceneDiscoveryImpl(const PREGEN_NS::SceneDiscovery& InSceneDiscovery)
				: PregenSceneDiscovery(InSceneDiscovery)
			{
			}

			explicit FSceneDiscoveryImpl(PREGEN_NS::SceneDiscovery&& InSceneDiscovery)
				: PregenSceneDiscovery(MoveTemp(InSceneDiscovery))
			{
			}

			TUsdStore<PREGEN_NS::SceneDiscovery> PregenSceneDiscovery;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal

	FSceneDiscovery::FSceneDiscovery(const UE::FUsdStage& Stage)
	{
#if USE_USD_SDK
		PREGEN_NS::SceneDiscovery NativeSceneDiscovery = [&]() -> PREGEN_NS::SceneDiscovery
			{
				FScopedUsdAllocs UsdAllocs;
				return PREGEN_NS::SceneDiscovery{ Stage };
			}();

		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneDiscoveryImpl>(MoveTemp(NativeSceneDiscovery));
#endif	  // #if USE_USD_SDK
	}

	FSceneDiscovery::FSceneDiscovery(const UE::FUsdStage& Stage, const FPregenDiscoveryOptions& Options)
	{
#if USE_USD_SDK
		PREGEN_NS::SceneDiscovery NativeSceneDiscovery = [&]() -> PREGEN_NS::SceneDiscovery
		{
			FScopedUsdAllocs UsdAllocs;

			pxr::TfTokenVector Purposes;
			Purposes.reserve(Options.Purposes.Num());
			for (const FString& Purpose : Options.Purposes)
			{
				Purposes.emplace_back(TCHAR_TO_UTF8(*Purpose));
			}

			pxr::TfTokenVector ExcludeVariantSets;
			ExcludeVariantSets.reserve(Options.ExcludeVariantSets.Num());
			for (const FString& Entry : Options.ExcludeVariantSets)
			{
				ExcludeVariantSets.emplace_back(TCHAR_TO_UTF8(*Entry));
			}

			PREGEN_NS::DiscoveryOptions SceneTrackerOptions{
				static_cast<PREGEN_NS::DiscoveryMode>(Options.DiscoveryMode),
				TCHAR_TO_UTF8(*Options.DiscoveryPluginName),
				TCHAR_TO_UTF8(*Options.DefinitionPrefix),
				UE::FSdfPath{ *Options.InitialPath },
				Purposes,
				ExcludeVariantSets,
				static_cast<PREGEN_NS::IdentifierFallbackMode>(Options.AssetIdentifierFallback),
				static_cast<PREGEN_NS::VersionFallbackMode>(Options.AssetVersionFallback)
			};

			return PREGEN_NS::SceneDiscovery{ Stage, SceneTrackerOptions };
		}();

		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneDiscoveryImpl>(MoveTemp(NativeSceneDiscovery));
#endif	  // #if USE_USD_SDK
	}

	FSceneDiscovery::FSceneDiscovery(const FSceneDiscovery& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneDiscoveryImpl>(Other.Impl->PregenSceneDiscovery.Get());
#endif	  // #if USE_USD_SDK
	}

	FSceneDiscovery::FSceneDiscovery(FSceneDiscovery&& Other) = default;

	FSceneDiscovery::~FSceneDiscovery()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FSceneDiscovery& FSceneDiscovery::operator=(const FSceneDiscovery& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneDiscoveryImpl>(Other.Impl->PregenSceneDiscovery.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FSceneDiscovery& FSceneDiscovery::operator=(FSceneDiscovery&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);
		return *this;
	}

#if USE_USD_SDK
	FSceneDiscovery::FSceneDiscovery(const PREGEN_NS::SceneDiscovery& InSceneDiscovery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneDiscoveryImpl>(InSceneDiscovery);
	}

	FSceneDiscovery::FSceneDiscovery(PREGEN_NS::SceneDiscovery&& InSceneDiscovery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneDiscoveryImpl>(MoveTemp(InSceneDiscovery));
	}

	FSceneDiscovery& FSceneDiscovery::operator=(const PREGEN_NS::SceneDiscovery& InSceneDiscovery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneDiscoveryImpl>(InSceneDiscovery);
		return *this;
	}

	FSceneDiscovery& FSceneDiscovery::operator=(PREGEN_NS::SceneDiscovery&& InSceneDiscovery)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneDiscoveryImpl>(MoveTemp(InSceneDiscovery));
		return *this;
	}

	FSceneDiscovery::operator PREGEN_NS::SceneDiscovery& ()
	{
		return Impl->PregenSceneDiscovery.Get();
	}

	FSceneDiscovery::operator const PREGEN_NS::SceneDiscovery& () const
	{
		return Impl->PregenSceneDiscovery.Get();
	}
#endif	  // #if USE_USD_SDK

	FTargetData FSceneDiscovery::GetTargetData(const FTargetUid& TargetUid)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return FTargetData{ Impl->PregenSceneDiscovery.Get().GetTargetData(TargetUid) };
#else
		return FTargetData{};
#endif	  // #if USE_USD_SDK
	}

	bool FSceneDiscovery::TraverseAndFindTargets(ResultMap& Results)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		PREGEN_NS::SceneDiscovery::ResultMap UsdResults;
		const bool bResult = Impl->PregenSceneDiscovery.Get().TraverseAndFindTargets(UsdResults);
		if (!bResult)
		{
			return false;
		}

		Results.Reset();

		for (const auto& Pair : UsdResults)
		{
			UE::FSdfPath Path{ Pair.first };

			TArray<FTargetUid> Targets;
			Targets.Reserve(Pair.second.size());

			for (const PREGEN_NS::TargetUid& Target : Pair.second)
			{
				Targets.Add(FTargetUid{ Target });
			}

			Results.Add(Path, MoveTemp(Targets));
		}

		return true;
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FSceneDiscovery::SaveDiscoveryData(const FString& Filename)
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return Impl->PregenSceneDiscovery.Get().SaveDiscoveryData(TCHAR_TO_UTF8(*Filename));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE::UsdPregen