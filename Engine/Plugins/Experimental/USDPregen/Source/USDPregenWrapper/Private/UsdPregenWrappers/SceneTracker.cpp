// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/SceneTracker.h"

#include "UsdPregen/pregen.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"

#include "USDMemory.h"

#include <type_traits>

#if USE_USD_SDK
#include "UsdPregen/target.h"
#include "UsdPregen/sceneTracker.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	namespace Internal
	{
		class FTrackedPrimImpl
		{
		public:
			FTrackedPrimImpl() = default;

#if USE_USD_SDK
			explicit FTrackedPrimImpl(PREGEN_NS::TrackedPrim&& InTrackedPrim)
			{
				PregenTrackedPrim = MakeUnique<TUsdStore<PREGEN_NS::TrackedPrim>>(MoveTemp(InTrackedPrim));
			}

			TUniquePtr<TUsdStore<PREGEN_NS::TrackedPrim>> PregenTrackedPrim;
#endif	  // #if USE_USD_SDK
		};

		template<typename PtrType>
		class FSceneTrackerImpl
		{
		public:
			FSceneTrackerImpl() = default;

			explicit FSceneTrackerImpl(const PtrType& InSceneTracker)
				: PregenSceneTracker(InSceneTracker)
			{
			}

			explicit FSceneTrackerImpl(PtrType&& InSceneTracker)
				: PregenSceneTracker(MoveTemp(InSceneTracker))
			{
			}

			PtrType& GetInner()
			{
				return PregenSceneTracker.Get();
			}

			const PtrType& GetInner() const
			{
				return PregenSceneTracker.Get();
			}

		private:
			TUsdStore<PtrType> PregenSceneTracker;
		};
	}	 // namespace Internal

	FTrackedPrim::FTrackedPrim()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTrackedPrimImpl>();
	}

	FTrackedPrim::FTrackedPrim(FTrackedPrim&& Other) = default;

	FTrackedPrim::~FTrackedPrim()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FTrackedPrim& FTrackedPrim::operator=(FTrackedPrim&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);
		return *this;
	}

#if USE_USD_SDK
	FTrackedPrim::FTrackedPrim(PREGEN_NS::TrackedPrim&& InTrackedPrim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTrackedPrimImpl>(MoveTemp(InTrackedPrim));
	}

	FTrackedPrim& FTrackedPrim::operator=(PREGEN_NS::TrackedPrim&& InTrackedPrim)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTrackedPrimImpl>(MoveTemp(InTrackedPrim));
		return *this;
	}
#endif	  // #if USE_USD_SDK

	bool FTrackedPrim::HasUnprocessedPermutations() const
	{
#if USE_USD_SDK
		if (Impl && Impl->PregenTrackedPrim)
		{
			return Impl->PregenTrackedPrim->Get().HasUnprocessedPermutations();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	bool FTrackedPrim::PrepareNextPermutation()
	{
#if USE_USD_SDK
		if (Impl && Impl->PregenTrackedPrim)
		{
			return Impl->PregenTrackedPrim->Get().PrepareNextPermutation();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::FSceneTrackerBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>();
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::FSceneTrackerBase(const FSceneTracker& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::FSceneTrackerBase(FSceneTracker&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::FSceneTrackerBase(const FSceneTrackerWeak& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::FSceneTrackerBase(FSceneTrackerWeak&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>& FSceneTrackerBase<PtrType>::operator=(const FSceneTracker& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
		return *this;
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>& FSceneTrackerBase<PtrType>::operator=(FSceneTracker&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
		return *this;
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>& FSceneTrackerBase<PtrType>::operator=(const FSceneTrackerWeak& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
		return *this;
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>& FSceneTrackerBase<PtrType>::operator=(FSceneTrackerWeak&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
		return *this;
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::~FSceneTrackerBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Internal::ToStrongPtr(Impl->GetInner()));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	template<typename OtherPtrType>
	bool FSceneTrackerBase<PtrType>::operator==(const FSceneTrackerBase<OtherPtrType>& Other) const
	{
#if USE_USD_SDK
		return Internal::ToStrongPtr(Impl->GetInner()) == Internal::ToStrongPtr(Other.Impl->GetInner());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	template USDPREGENWRAPPER_API bool FSceneTracker::operator==(const FSceneTracker& Other) const;
	template USDPREGENWRAPPER_API bool FSceneTracker::operator==(const FSceneTrackerWeak& Other) const;
	template USDPREGENWRAPPER_API bool FSceneTrackerWeak::operator==(const FSceneTracker& Other) const;
	template USDPREGENWRAPPER_API bool FSceneTrackerWeak::operator==(const FSceneTrackerWeak& Other) const;

#if USE_USD_SDK
	template<typename PtrType>
	FSceneTrackerBase<PtrType>::FSceneTrackerBase(const PREGEN_NS::SceneTrackerRefPtr& InSceneTracker)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(InSceneTracker)
		);
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::FSceneTrackerBase(PREGEN_NS::SceneTrackerRefPtr&& InSceneTracker)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(InSceneTracker))
		);
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::FSceneTrackerBase(const PREGEN_NS::SceneTrackerWeakPtr& InSceneTracker)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(InSceneTracker)
		);
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::FSceneTrackerBase(PREGEN_NS::SceneTrackerWeakPtr&& InSceneTracker)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FSceneTrackerImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(InSceneTracker))
		);
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::operator PtrType& ()
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::operator const PtrType& () const
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::operator PREGEN_NS::SceneTrackerRefPtr() const
	{
		return Internal::ToStrongPtr(Impl->GetInner());
	}

	template<typename PtrType>
	FSceneTrackerBase<PtrType>::operator PREGEN_NS::SceneTrackerWeakPtr() const
	{
		if constexpr (std::is_same_v<PtrType, PREGEN_NS::SceneTrackerWeakPtr>)
		{
			return Impl->GetInner();
		}
		else
		{
			return PREGEN_NS::SceneTrackerWeakPtr{ Impl->GetInner() };
		}
	}
#endif	  // #if USE_USD_SDK

	template<typename PtrType>
	FPregenDiscoveryOptions FSceneTrackerBase<PtrType>::GetOptions() const
	{
#if USE_USD_SDK
		if (PREGEN_NS::SceneTrackerRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			const PREGEN_NS::DiscoveryOptions& Options = Ptr->GetOptions();
			return FPregenDiscoveryOptions{
				static_cast<EPregenDiscoveryMode>(Options.discoveryMode),
				UTF8_TO_TCHAR(Options.discoveryPluginName.c_str())
			};
		}
#endif	  // #if USE_USD_SDK

		return FPregenDiscoveryOptions{};
	}

	template<typename PtrType>
	FTargetData FSceneTrackerBase<PtrType>::GetTargetData(const FTargetUid& TargetId) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::SceneTrackerRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			FScopedUsdAllocs Allocs;
			return FTargetData{ Ptr->GetTargetData(TargetId) };
		}
#endif	  // #if USE_USD_SDK

		return FTargetData{};
	}

	template<typename PtrType>
	bool FSceneTrackerBase<PtrType>::HasErrors() const
	{
#if USE_USD_SDK
		if (PREGEN_NS::SceneTrackerRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return Ptr->HasErrors();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	bool FSceneTrackerBase<PtrType>::SaveDataLayer(const FString& Filename) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::SceneTrackerRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			FScopedUsdAllocs Allocs;
			return Ptr->SaveDataLayer(TCHAR_TO_UTF8(*Filename));
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	FTrackedPrim FSceneTrackerBase<PtrType>::StartTrackingPrim(const UE::FUsdPrim& Prim)
	{
#if USE_USD_SDK
		if (PREGEN_NS::SceneTrackerRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			FScopedUsdAllocs Allocs;
			return FTrackedPrim{ Ptr->StartTrackingPrim(Prim) };
		}
#endif	  // #if USE_USD_SDK

		return FTrackedPrim{};
	}

	template<typename PtrType>
	void FSceneTrackerBase<PtrType>::SetTargetCreatedCallback(
		TFunction<void(const UE::FSdfPath&, const FTargetUid&)> Callback
	)
	{
#if USE_USD_SDK
		if (PREGEN_NS::SceneTrackerRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			auto SharedCallback =
				MakeShared<TFunction<void(const UE::FSdfPath&, const FTargetUid&)>, ESPMode::ThreadSafe>(
					MoveTemp(Callback)
				);

			Ptr->SetTargetCreatedCallback(
				[SharedCallback](const pxr::SdfPath& Path, const PREGEN_NS::TargetUid& TargetUid)
				{
					(*SharedCallback)(UE::FSdfPath{ Path }, FTargetUid{ TargetUid });
				}
			);
		}
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	void FSceneTrackerBase<PtrType>::RemoveTargetCreatedCallback()
	{
#if USE_USD_SDK
		if (PREGEN_NS::SceneTrackerRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			Ptr->RemoveTargetCreatedCallback();
		}
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	template class FSceneTrackerBase<PREGEN_NS::SceneTrackerRefPtr>;
	template class FSceneTrackerBase<PREGEN_NS::SceneTrackerWeakPtr>;
#else
	template class FSceneTrackerBase<FDummyRefPtrType>;
	template class FSceneTrackerBase<FDummyWeakPtrType>;
#endif	  // #if USE_USD_SDK
}	 // namespace UE::UsdPregen