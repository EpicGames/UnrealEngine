// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/StoragePlugin.h"

#include "UsdPregenWrappers/ExtAssetDefinition.h"
#include "UsdPregenWrappers/Manifest.h"
#include "UsdPregenWrappers/ManifestTypes.h"
#include "UsdPregenWrappers/Target.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/manifest.h"
#include "UsdPregen/manifestTypes.h"
#include "UsdPregen/storagePlugin.h"
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	namespace Internal
	{
		template<typename PtrType>
		class FStoragePluginImpl
		{
		public:
			FStoragePluginImpl() = default;

			explicit FStoragePluginImpl(const PtrType& InStorage)
				: PregenStorage(InStorage)
			{
			}

			explicit FStoragePluginImpl(PtrType&& InStorage)
				: PregenStorage(MoveTemp(InStorage))
			{
			}

			PtrType& GetInner()
			{
				return PregenStorage.Get();
			}

			const PtrType& GetInner() const
			{
				return PregenStorage.Get();
			}

		private:
			TUsdStore<PtrType> PregenStorage;
		};
	}	 // namespace Internal

	template<typename PtrType>
	FStoragePluginBase<PtrType>::FStoragePluginBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>();
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::FStoragePluginBase(const FStoragePlugin& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::FStoragePluginBase(FStoragePlugin&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::FStoragePluginBase(const FStoragePluginWeak& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::FStoragePluginBase(FStoragePluginWeak&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>& FStoragePluginBase<PtrType>::operator=(const FStoragePlugin& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
		return *this;
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>& FStoragePluginBase<PtrType>::operator=(FStoragePlugin&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
		return *this;
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>& FStoragePluginBase<PtrType>::operator=(const FStoragePluginWeak& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
		return *this;
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>& FStoragePluginBase<PtrType>::operator=(FStoragePluginWeak&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
		return *this;
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::~FStoragePluginBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Internal::ToStrongPtr(Impl->GetInner()));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	template<typename OtherPtrType>
	bool FStoragePluginBase<PtrType>::operator==(const FStoragePluginBase<OtherPtrType>& Other) const
	{
#if USE_USD_SDK
		return Internal::ToStrongPtr(Impl->GetInner()) == Internal::ToStrongPtr(Other.Impl->GetInner());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	template USDPREGENWRAPPER_API bool FStoragePlugin::operator==(const FStoragePlugin& Other) const;
	template USDPREGENWRAPPER_API bool FStoragePlugin::operator==(const FStoragePluginWeak& Other) const;
	template USDPREGENWRAPPER_API bool FStoragePluginWeak::operator==(const FStoragePlugin& Other) const;
	template USDPREGENWRAPPER_API bool FStoragePluginWeak::operator==(const FStoragePluginWeak& Other) const;

#if USE_USD_SDK
	template<typename PtrType>
	FStoragePluginBase<PtrType>::FStoragePluginBase(const PREGEN_NS::StoragePluginRefPtr& InStorage)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(InStorage)
		);
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::FStoragePluginBase(PREGEN_NS::StoragePluginRefPtr&& InStorage)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(InStorage))
		);
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::FStoragePluginBase(const PREGEN_NS::StoragePluginWeakPtr& InStorage)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(InStorage)
		);
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::FStoragePluginBase(PREGEN_NS::StoragePluginWeakPtr&& InStorage)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FStoragePluginImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(InStorage))
		);
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::operator PtrType& ()
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::operator const PtrType& () const
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::operator PREGEN_NS::StoragePluginRefPtr() const
	{
		return Internal::ToStrongPtr(Impl->GetInner());
	}

	template<typename PtrType>
	FStoragePluginBase<PtrType>::operator PREGEN_NS::StoragePluginWeakPtr() const
	{
		if constexpr (std::is_same_v<PtrType, PREGEN_NS::StoragePluginWeakPtr>)
		{
			return Impl->GetInner();
		}
		else
		{
			return PREGEN_NS::StoragePluginWeakPtr{ Impl->GetInner() };
		}
	}
#endif	  // #if USE_USD_SDK

	template<typename PtrType>
	bool FStoragePluginBase<PtrType>::IsValid() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Internal::ToStrongPtr(Impl->GetInner()));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	bool FStoragePluginBase<PtrType>::Initialize() const
	{
#if USE_USD_SDK
		if (PREGEN_NS::StoragePluginRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return Ptr->Initialize();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	bool FStoragePluginBase<PtrType>::Shutdown() const
	{
#if USE_USD_SDK
		if (PREGEN_NS::StoragePluginRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return Ptr->Shutdown();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	FManifestLoadResult FStoragePluginBase<PtrType>::LoadManifestPayload(const FTargetUid& TargetUid) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::StoragePluginRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return FManifestLoadResult{ Ptr->LoadManifestPayload(TargetUid) };
		}
#endif	  // #if USE_USD_SDK

		return FManifestLoadResult{};
	}

	template<typename PtrType>
	FManifestSaveResult FStoragePluginBase<PtrType>::StoreManifestPayload(
		const FTargetUid& TargetUid,
		const FManifestPayload& Payload
	) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::StoragePluginRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return FManifestSaveResult{ Ptr->StoreManifestPayload(TargetUid, Payload) };
		}
#endif	  // #if USE_USD_SDK

		return FManifestSaveResult{};
	}

	template<typename PtrType>
	FManifestSaveResult FStoragePluginBase<PtrType>::PersistManifestPayload(const FTargetUid& TargetUid) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::StoragePluginRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return FManifestSaveResult{ Ptr->PersistManifestPayload(TargetUid) };
		}
#endif	  // #if USE_USD_SDK

		return FManifestSaveResult{};
	}

	template<typename PtrType>
	FManifestPayload FStoragePluginBase<PtrType>::SerializeManifest(const FManifest& Manifest) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::StoragePluginRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return FManifestPayload{ Ptr->SerializeManifest(Manifest) };
		}
#endif	  // #if USE_USD_SDK

		return FManifestPayload{};
	}

	template<typename PtrType>
	FManifest FStoragePluginBase<PtrType>::DeserializeManifestPayload(const FManifestPayload& Payload) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::StoragePluginRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return FManifest{ Ptr->DeserializeManifestPayload(Payload) };
		}
#endif	  // #if USE_USD_SDK

		return FManifest{};
	}

	template<typename PtrType>
	FString FStoragePluginBase<PtrType>::GetNameForUAsset(
		const FTargetUid& TargetUid,
		const TArray<FExtAssetDefinition>& Definitions,
		const FString& AssetType
	) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::StoragePluginRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			FScopedUsdAllocs UsdAllocs;

			std::vector<const PREGEN_NS::ExtAssetDefinition*> NativeDefinitions;
			NativeDefinitions.reserve(Definitions.Num());

			for (const FExtAssetDefinition& Definition : Definitions)
			{
				if (const PREGEN_NS::ExtAssetDefinition* NativeDefinition = Definition)
				{
					NativeDefinitions.push_back(NativeDefinition);
				}
			}

			return UTF8_TO_TCHAR(
				Ptr->GetNameForUAsset(
					TargetUid,
					NativeDefinitions,
					TCHAR_TO_UTF8(*AssetType)
				).c_str()
			);
		}
#endif	  // #if USE_USD_SDK

		return FString{};
	}

	template<typename PtrType>
	FString FStoragePluginBase<PtrType>::GetPackageSubPathForUAsset(
		const FTargetUid& TargetUid,
		const TArray<FExtAssetDefinition>& Definitions,
		const FString& AssetType
	) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::StoragePluginRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			FScopedUsdAllocs UsdAllocs;

			std::vector<const PREGEN_NS::ExtAssetDefinition*> NativeDefinitions;
			NativeDefinitions.reserve(Definitions.Num());

			for (const FExtAssetDefinition& Definition : Definitions)
			{
				if (const PREGEN_NS::ExtAssetDefinition* NativeDefinition = Definition)
				{
					NativeDefinitions.push_back(NativeDefinition);
				}
			}

			return UTF8_TO_TCHAR(
				Ptr->GetPackageSubPathForUAsset(
					TargetUid,
					NativeDefinitions,
					TCHAR_TO_UTF8(*AssetType)
				).c_str()
			);
		}
#endif	  // #if USE_USD_SDK

		return FString{};
	}

	template<typename PtrType>
	FString FStoragePluginBase<PtrType>::GetPathForManifest(const FTargetUid& TargetUid) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::StoragePluginRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			FScopedUsdAllocs UsdAllocs;
			return UTF8_TO_TCHAR(Ptr->GetPathForManifest(TargetUid).c_str());
		}
#endif	  // #if USE_USD_SDK

		return FString{};
	}

#if USE_USD_SDK
	template class FStoragePluginBase<PREGEN_NS::StoragePluginRefPtr>;
	template class FStoragePluginBase<PREGEN_NS::StoragePluginWeakPtr>;
#else
	template class FStoragePluginBase<FDummyRefPtrType>;
	template class FStoragePluginBase<FDummyWeakPtrType>;
#endif	  // #if USE_USD_SDK
}	 // namespace UE::UsdPregen