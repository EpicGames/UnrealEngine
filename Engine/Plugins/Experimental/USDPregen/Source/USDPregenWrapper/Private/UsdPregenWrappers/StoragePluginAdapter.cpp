// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/StoragePluginAdapter.h"

#include "UsdPregenWrappers/ExtAssetDefinition.h"
#include "UsdPregenWrappers/Manifest.h"
#include "UsdPregenWrappers/ManifestTypes.h"
#include "UsdPregenWrappers/Target.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/manifest.h"
#include "UsdPregen/manifestTypes.h"
#include "UsdPregen/storagePlugin.h"
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
#if USE_USD_SDK
	FStoragePluginAdapter::FStoragePluginAdapter(
		const TSharedRef<IStoragePlugin, ESPMode::ThreadSafe>& InStoragePlugin
	)
		: StoragePlugin(InStoragePlugin)
	{
	}

	FStoragePluginAdapter::~FStoragePluginAdapter() = default;

	PREGEN_NS::ManifestLoadResult FStoragePluginAdapter::LoadManifestPayload(
		const PREGEN_NS::TargetUid& TargetUid
	)
	{
		return StoragePlugin->LoadManifestPayload(FTargetUid{ TargetUid });
	}

	PREGEN_NS::ManifestSaveResult FStoragePluginAdapter::StoreManifestPayload(
		const PREGEN_NS::TargetUid& TargetUid,
		const PREGEN_NS::ManifestPayload& Payload
	)
	{
		return StoragePlugin->StoreManifestPayload(
			FTargetUid{ TargetUid },
			FManifestPayload{ Payload }
		);
	}

	PREGEN_NS::ManifestSaveResult FStoragePluginAdapter::PersistManifestPayload(
		const PREGEN_NS::TargetUid& TargetUid
	)
	{
		return StoragePlugin->PersistManifestPayload(FTargetUid{ TargetUid });
	}

	PREGEN_NS::ManifestPayload FStoragePluginAdapter::SerializeManifest(
		const PREGEN_NS::Manifest& Manifest
	)
	{
		return StoragePlugin->SerializeManifest(FManifest{ Manifest });
	}

	PREGEN_NS::Manifest FStoragePluginAdapter::DeserializeManifestPayload(
		const PREGEN_NS::ManifestPayload& Payload
	)
	{
		FManifest Result;
		{
			FScopedUnrealAllocs UnrealAllocs;
			Result = StoragePlugin->DeserializeManifestPayload(FManifestPayload{ Payload });
		}

		{
			FScopedUsdAllocs UsdAllocs;
			return static_cast<PREGEN_NS::Manifest>(Result);
		}
	}

	std::string FStoragePluginAdapter::GetNameForUAsset(
		const PREGEN_NS::TargetUid& TargetUid,
		const std::vector<const PREGEN_NS::ExtAssetDefinition*>& Definitions,
		const std::string& AssetType
	)
	{
		FString Result;
		{
			FScopedUnrealAllocs UnrealAllocs;

			TArray<FExtAssetDefinition> WrappedDefinitions;
			WrappedDefinitions.Reserve(static_cast<int32>(Definitions.size()));

			for (const PREGEN_NS::ExtAssetDefinition* Definition : Definitions)
			{
				WrappedDefinitions.Add(FExtAssetDefinition{ Definition });
			}

			Result = StoragePlugin->GetNameForUAsset(
				FTargetUid{ TargetUid },
				WrappedDefinitions,
				UTF8_TO_TCHAR(AssetType.c_str())
			);
		}

		{
			FScopedUsdAllocs UsdAllocs;
			return TCHAR_TO_UTF8(*Result);
		}
	}

	std::string FStoragePluginAdapter::GetPackageSubPathForUAsset(
		const PREGEN_NS::TargetUid& TargetUid,
		const std::vector<const PREGEN_NS::ExtAssetDefinition*>& Definitions,
		const std::string& AssetType
	)
	{
		FString Result;
		{
			FScopedUnrealAllocs UnrealAllocs;

			TArray<FExtAssetDefinition> WrappedDefinitions;
			WrappedDefinitions.Reserve(static_cast<int32>(Definitions.size()));

			for (const PREGEN_NS::ExtAssetDefinition* Definition : Definitions)
			{
				WrappedDefinitions.Add(FExtAssetDefinition{ Definition });
			}

			Result = StoragePlugin->GetPackageSubPathForUAsset(
				FTargetUid{ TargetUid },
				WrappedDefinitions,
				UTF8_TO_TCHAR(AssetType.c_str())
			);
		}

		{
			FScopedUsdAllocs UsdAllocs;
			return TCHAR_TO_UTF8(*Result);
		}
	}

	std::string FStoragePluginAdapter::GetPathForManifest(
		const PREGEN_NS::TargetUid& TargetUid
	)
	{
		FString Result;
		{
			FScopedUnrealAllocs UnrealAllocs;
			Result = StoragePlugin->GetPathForManifest(FTargetUid{ TargetUid });
		}

		{
			FScopedUsdAllocs UsdAllocs;
			return TCHAR_TO_UTF8(*Result);
		}
	}

	PREGEN_NS::StoragePluginRefPtr MakeStoragePluginAdapter(
		const TSharedRef<IStoragePlugin, ESPMode::ThreadSafe>& InStoragePlugin
	)
	{
		FScopedUsdAllocs UsdAllocs;
		return PREGEN_NS::StoragePluginRefPtr{ new FStoragePluginAdapter{ InStoragePlugin } };
	}
#endif	  // #if USE_USD_SDK
}	 // namespace UE::UsdPregen
