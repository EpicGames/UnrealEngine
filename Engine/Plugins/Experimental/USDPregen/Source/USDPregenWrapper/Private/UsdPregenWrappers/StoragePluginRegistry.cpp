// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/StoragePluginRegistry.h"

#include "UsdPregenWrappers/IStoragePlugin.h"
#include "UsdPregenWrappers/StoragePluginAdapter.h"

#include "USDMemory.h"

#if USE_USD_SDK
#include "UsdPregen/storagePlugin.h"
#include "UsdPregen/storageOptions.h"
#include "UsdPregen/storagePluginRegistry.h"
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
#if USE_USD_SDK
	namespace
	{
		PREGEN_NS::StorageOptions ToNativeOptions(const FPregenStorageOptions& Options)
		{
			PREGEN_NS::StorageOptions NativeOptions;
			NativeOptions.storagePluginName = TCHAR_TO_UTF8(*Options.StoragePluginName);
			NativeOptions.manifestDir = TCHAR_TO_UTF8(*Options.ManifestDir);
			NativeOptions.packageSubPathTemplate = TCHAR_TO_UTF8(*Options.PackageSubPathTemplate);
			return NativeOptions;
		}

		FPregenStorageOptions ToWrappedOptions(const PREGEN_NS::StorageOptions& NativeOptions)
		{
			FPregenStorageOptions Options;
			Options.StoragePluginName = UTF8_TO_TCHAR(NativeOptions.storagePluginName.c_str());
			Options.ManifestDir = UTF8_TO_TCHAR(NativeOptions.manifestDir.c_str());
			Options.PackageSubPathTemplate = UTF8_TO_TCHAR(NativeOptions.packageSubPathTemplate.c_str());
			return Options;
		}
	}

	FStoragePluginRegistry::FStoragePluginRegistry(PREGEN_NS::StoragePluginRegistry* InRegistry)
		: PregenRegistry(InRegistry)
	{
	}

	FStoragePluginRegistry::operator PREGEN_NS::StoragePluginRegistry* ()
	{
		return PregenRegistry;
	}

	FStoragePluginRegistry::operator const PREGEN_NS::StoragePluginRegistry* () const
	{
		return PregenRegistry;
	}
#endif	  // #if USE_USD_SDK

	// static
	FStoragePluginRegistry FStoragePluginRegistry::GetInstance()
	{
#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;
		return FStoragePluginRegistry{ &PREGEN_NS::StoragePluginRegistry::GetInstance() };
#else
		return FStoragePluginRegistry{};
#endif	  // #if USE_USD_SDK
	}

	bool FStoragePluginRegistry::RegisterFactory(
		const FString& Name,
		FCreateStoragePluginFunction Factory
	)
	{
#if USE_USD_SDK
		if (!PregenRegistry)
		{
			return false;
		}

		FScopedUsdAllocs UsdAllocs;

		TSharedRef<FCreateStoragePluginFunction, ESPMode::ThreadSafe> SharedFactory =
			MakeShared<FCreateStoragePluginFunction, ESPMode::ThreadSafe>(MoveTemp(Factory));

		return PregenRegistry->RegisterFactory(
			TCHAR_TO_UTF8(*Name),
			[SharedFactory](const PREGEN_NS::StorageOptions& NativeOptions) -> PREGEN_NS::StoragePlugin*
			{
				// The native registry invokes us with native-allocator
				// scope; flip back to Unreal allocs while we run the
				// user's factory and build the adapter wrapper, then
				// switch back to native to return the raw pointer.
				TSharedRef<IStoragePlugin, ESPMode::ThreadSafe> StoragePlugin = [&]()
					{
						FScopedUnrealAllocs UnrealAllocs;
						return (*SharedFactory)(ToWrappedOptions(NativeOptions));
					}();
				FScopedUsdAllocs UsdAllocs;
				return new FStoragePluginAdapter{ StoragePlugin };
			}
		);
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FStoragePlugin FStoragePluginRegistry::Create(
		const FPregenStorageOptions& Options
	) const
	{
#if USE_USD_SDK
		if (!PregenRegistry)
		{
			return FStoragePlugin{};
		}

		FScopedUsdAllocs UsdAllocs;
		return FStoragePlugin{ PregenRegistry->Create(ToNativeOptions(Options)) };
#else
		return FStoragePlugin{};
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE::UsdPregen
