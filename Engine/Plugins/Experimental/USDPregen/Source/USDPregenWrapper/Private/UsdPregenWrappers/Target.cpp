// Copyright Epic Games, Inc. All Rights Reserved.

#include "UsdPregenWrappers/Target.h"

#include "UsdPregen/pregen.h"

#include "UsdPregenWrappers/ExtAssetDefinition.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"

#include "Templates/TypeHash.h"
#include "USDMemory.h"

#include <type_traits>

#if USE_USD_SDK
#include "UsdPregen/extAssetDefinition.h"
#include "UsdPregen/permutationOps.h"
#include "UsdPregen/target.h"
#include "UsdPregen/util.h"

#include "USDIncludesStart.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/sdf/path.h"
#include "USDIncludesEnd.h"

#include <optional>
#include <utility>
#include <vector>
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	namespace Internal
	{
		class FTargetUidImpl
		{
		public:
			FTargetUidImpl() = default;

#if USE_USD_SDK
			explicit FTargetUidImpl(const PREGEN_NS::TargetUid& InTargetUid)
				: PregenTargetUid(InTargetUid)
			{
			}

			explicit FTargetUidImpl(PREGEN_NS::TargetUid&& InTargetUid)
				: PregenTargetUid(MoveTemp(InTargetUid))
			{
			}

			TUsdStore<PREGEN_NS::TargetUid> PregenTargetUid;
#endif	  // #if USE_USD_SDK
		};

		class FTargetDefinitionEntryImpl
		{
		public:
			FTargetDefinitionEntryImpl() = default;

#if USE_USD_SDK
			explicit FTargetDefinitionEntryImpl(const PREGEN_NS::TargetDefinitionEntry& InTargetDefinitionEntry)
				: PregenTargetDefinitionEntry(InTargetDefinitionEntry)
			{
			}

			explicit FTargetDefinitionEntryImpl(PREGEN_NS::TargetDefinitionEntry&& InTargetDefinitionEntry)
				: PregenTargetDefinitionEntry(MoveTemp(InTargetDefinitionEntry))
			{
			}

			TUsdStore<PREGEN_NS::TargetDefinitionEntry> PregenTargetDefinitionEntry;
#endif	  // #if USE_USD_SDK
		};

		template<typename PtrType>
		class FTargetDataImpl
		{
		public:
			FTargetDataImpl() = default;

			explicit FTargetDataImpl(const PtrType& InTargetData)
				: PregenTargetData(InTargetData)
			{
			}

			explicit FTargetDataImpl(PtrType&& InTargetData)
				: PregenTargetData(MoveTemp(InTargetData))
			{
			}

			PtrType& GetInner()
			{
				return PregenTargetData.Get();
			}

			const PtrType& GetInner() const
			{
				return PregenTargetData.Get();
			}

		private:
			TUsdStore<PtrType> PregenTargetData;
		};
	}	 // namespace Internal

	FTargetUid::FTargetUid()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>();
	}

	FTargetUid::FTargetUid(const FTargetUid& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>(Other.Impl->PregenTargetUid.Get());
#endif	  // #if USE_USD_SDK
	}

	FTargetUid::FTargetUid(FTargetUid&& Other) = default;

	FTargetUid::~FTargetUid()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FTargetUid& FTargetUid::operator=(const FTargetUid& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>(Other.Impl->PregenTargetUid.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FTargetUid& FTargetUid::operator=(FTargetUid&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);
		return *this;
	}

	FTargetUid::FTargetUid(const FString& DefinitionUid)
	{
#if USE_USD_SDK
		PREGEN_NS::TargetUid NativeTargetUid;
		{
			FScopedUsdAllocs UsdAllocs;
			NativeTargetUid = PREGEN_NS::TargetUid{ TCHAR_TO_UTF8(*DefinitionUid) };
		}

		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>(MoveTemp(NativeTargetUid));
#else
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>();
#endif	  // #if USE_USD_SDK
	}

	FTargetUid::FTargetUid(const FString& DefinitionUid, const FString& PermutationUid)
	{
#if USE_USD_SDK

		PREGEN_NS::TargetUid NativeTargetUid;
		{
			FScopedUsdAllocs UsdAllocs;
			NativeTargetUid = PREGEN_NS::TargetUid{
				TCHAR_TO_UTF8(*DefinitionUid),
				TCHAR_TO_UTF8(*PermutationUid)
			};
		}

		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>(MoveTemp(NativeTargetUid));
#else
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>();
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FTargetUid::FTargetUid(const PREGEN_NS::TargetUid& InTargetUid)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>(InTargetUid);
	}

	FTargetUid::FTargetUid(PREGEN_NS::TargetUid&& InTargetUid)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>(MoveTemp(InTargetUid));
	}

	FTargetUid& FTargetUid::operator=(const PREGEN_NS::TargetUid& InTargetUid)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>(InTargetUid);
		return *this;
	}

	FTargetUid& FTargetUid::operator=(PREGEN_NS::TargetUid&& InTargetUid)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetUidImpl>(MoveTemp(InTargetUid));
		return *this;
	}

	FTargetUid::operator PREGEN_NS::TargetUid& ()
	{
		return Impl->PregenTargetUid.Get();
	}

	FTargetUid::operator const PREGEN_NS::TargetUid& () const
	{
		return Impl->PregenTargetUid.Get();
	}
#endif	  // #if USE_USD_SDK

	bool FTargetUid::operator==(const FTargetUid& Other) const
	{
#if USE_USD_SDK
		return Impl->PregenTargetUid.Get() == Other.Impl->PregenTargetUid.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	bool FTargetUid::operator!=(const FTargetUid& Other) const
	{
		return !(*this == Other);
	}

	bool FTargetUid::operator<(const FTargetUid& Other) const
	{
#if USE_USD_SDK
		return Impl->PregenTargetUid.Get() < Other.Impl->PregenTargetUid.Get();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FTargetUid::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Impl->PregenTargetUid.Get());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FString FTargetUid::GetDefinitionUid() const
	{
#if USE_USD_SDK
		return UTF8_TO_TCHAR(Impl->PregenTargetUid.Get().GetDefinitionUid().c_str());
#else
		return FString();
#endif	  // #if USE_USD_SDK
	}

	FString FTargetUid::GetPermutationUid() const
	{
#if USE_USD_SDK
		return UTF8_TO_TCHAR(Impl->PregenTargetUid.Get().GetPermutationUid().c_str());
#else
		return FString();
#endif	  // #if USE_USD_SDK
	}

	bool FTargetUid::HasPermutationUid() const
	{
#if USE_USD_SDK
		return Impl->PregenTargetUid.Get().HasPermutationUid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	FString FTargetUid::GetString() const
	{
#if USE_USD_SDK
		return UTF8_TO_TCHAR(Impl->PregenTargetUid.Get().GetString().c_str());
#else
		return FString();
#endif	  // #if USE_USD_SDK
	}

	bool FTargetUid::IsValid() const
	{
#if USE_USD_SDK
		return Impl->PregenTargetUid.Get().IsValid();
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	uint32 GetTypeHash(const FTargetUid& TargetUid)
	{
		uint32 Hash = GetTypeHash(TargetUid.GetDefinitionUid());

		if (TargetUid.HasPermutationUid())
		{
			Hash = HashCombine(Hash, GetTypeHash(TargetUid.GetPermutationUid()));
		}

		return Hash;
	}

	FTargetDefinitionEntry::FTargetDefinitionEntry()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDefinitionEntryImpl>();
	}

	FTargetDefinitionEntry::FTargetDefinitionEntry(const FTargetDefinitionEntry& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDefinitionEntryImpl>(Other.Impl->PregenTargetDefinitionEntry.Get());
#endif	  // #if USE_USD_SDK
	}

	FTargetDefinitionEntry::FTargetDefinitionEntry(FTargetDefinitionEntry&& Other) = default;

	FTargetDefinitionEntry::~FTargetDefinitionEntry()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FTargetDefinitionEntry& FTargetDefinitionEntry::operator=(const FTargetDefinitionEntry& Other)
	{
#if USE_USD_SDK
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDefinitionEntryImpl>(Other.Impl->PregenTargetDefinitionEntry.Get());
#endif	  // #if USE_USD_SDK
		return *this;
	}

	FTargetDefinitionEntry& FTargetDefinitionEntry::operator=(FTargetDefinitionEntry&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MoveTemp(Other.Impl);
		return *this;
	}

	FTargetDefinitionEntry::FTargetDefinitionEntry(const FString& DefinitionUid, const UE::FSdfPath& ScenePath)
	{
#if USE_USD_SDK

		PREGEN_NS::TargetDefinitionEntry NativeTargetDefinitionEntry;
		{
			FScopedUsdAllocs UsdAllocs;
			NativeTargetDefinitionEntry = PREGEN_NS::TargetDefinitionEntry{ TCHAR_TO_UTF8(*DefinitionUid), ScenePath };
		}

		FScopedUnrealAllocs UnrealAllocs;		
		Impl = MakeUnique<Internal::FTargetDefinitionEntryImpl>(NativeTargetDefinitionEntry);
#else
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDefinitionEntryImpl>();
#endif	  // #if USE_USD_SDK
	}

#if USE_USD_SDK
	FTargetDefinitionEntry::FTargetDefinitionEntry(const PREGEN_NS::TargetDefinitionEntry& InTargetDefinitionEntry)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDefinitionEntryImpl>(InTargetDefinitionEntry);
	}

	FTargetDefinitionEntry::FTargetDefinitionEntry(PREGEN_NS::TargetDefinitionEntry&& InTargetDefinitionEntry)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDefinitionEntryImpl>(MoveTemp(InTargetDefinitionEntry));
	}

	FTargetDefinitionEntry& FTargetDefinitionEntry::operator=(const PREGEN_NS::TargetDefinitionEntry& InTargetDefinitionEntry)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDefinitionEntryImpl>(InTargetDefinitionEntry);
		return *this;
	}

	FTargetDefinitionEntry& FTargetDefinitionEntry::operator=(PREGEN_NS::TargetDefinitionEntry&& InTargetDefinitionEntry)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDefinitionEntryImpl>(MoveTemp(InTargetDefinitionEntry));
		return *this;
	}

	FTargetDefinitionEntry::operator PREGEN_NS::TargetDefinitionEntry& ()
	{
		return Impl->PregenTargetDefinitionEntry.Get();
	}

	FTargetDefinitionEntry::operator const PREGEN_NS::TargetDefinitionEntry& () const
	{
		return Impl->PregenTargetDefinitionEntry.Get();
	}
#endif	  // #if USE_USD_SDK

	FExtAssetDefinition FTargetDefinitionEntry::GetDefinition() const
	{
#if USE_USD_SDK
		if (const PREGEN_NS::ExtAssetDefinition* Definition = Impl->PregenTargetDefinitionEntry.Get().GetDefinition())
		{
			return FExtAssetDefinition{ Definition };
		}
#endif	  // #if USE_USD_SDK

		return FExtAssetDefinition{};
	}

	UE::FSdfPath FTargetDefinitionEntry::GetScenePath() const
	{
#if USE_USD_SDK
		return UE::FSdfPath{ Impl->PregenTargetDefinitionEntry.Get().GetScenePath() };
#else
		return UE::FSdfPath{};
#endif	  // #if USE_USD_SDK
	}

	TArray<FPermutationOp> FTargetDefinitionEntry::GetPermutationOps() const
	{
		TArray<FPermutationOp> Result;

#if USE_USD_SDK
		FScopedUsdAllocs UsdAllocs;

		const PREGEN_NS::PermutationOpVector& Ops
			= Impl->PregenTargetDefinitionEntry.Get().GetPermutationOps();
		Result.Reserve(static_cast<int32>(Ops.size()));

		for (const PREGEN_NS::PermutationOpRefPtr& Op : Ops)
		{
			const PREGEN_NS::internal::util::SerializedOp Serialized = PREGEN_NS::internal::util::ToSerializedOp(Op);

			FPermutationOp Wrapped;
			Wrapped.TypeName = UTF8_TO_TCHAR(Serialized.typeName.c_str());
			Wrapped.Opargs.Reserve(static_cast<int32>(Serialized.args.size()));
			for (const PREGEN_NS::internal::util::SerializedOpArg& Arg : Serialized.args)
			{
				FPermutationOpArg WrappedArg;
				WrappedArg.Name = UTF8_TO_TCHAR(Arg.name.c_str());
				WrappedArg.TypeName = UTF8_TO_TCHAR(Arg.typeName.c_str());
				WrappedArg.Value = UTF8_TO_TCHAR(Arg.value.c_str());
				Wrapped.Opargs.Add(MoveTemp(WrappedArg));
			}

			Result.Add(MoveTemp(Wrapped));
		}
#endif	  // #if USE_USD_SDK

		return Result;
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::FTargetDataBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>();
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::FTargetDataBase(const FTargetData& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::FTargetDataBase(FTargetData&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::FTargetDataBase(const FTargetDataWeak& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::FTargetDataBase(FTargetDataWeak&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>& FTargetDataBase<PtrType>::operator=(const FTargetData& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
		return *this;
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>& FTargetDataBase<PtrType>::operator=(FTargetData&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
		return *this;
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>& FTargetDataBase<PtrType>::operator=(const FTargetDataWeak& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(Other.Impl->GetInner())
		);
		return *this;
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>& FTargetDataBase<PtrType>::operator=(FTargetDataWeak&& Other)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(Other.Impl->GetInner()))
		);
		return *this;
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::~FTargetDataBase()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::operator bool() const
	{
#if USE_USD_SDK
		return static_cast<bool>(Internal::ToStrongPtr(Impl->GetInner()));
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	template<typename PtrType>
	template<typename OtherPtrType>
	bool FTargetDataBase<PtrType>::operator==(const FTargetDataBase<OtherPtrType>& Other) const
	{
#if USE_USD_SDK
		return Internal::ToStrongPtr(Impl->GetInner()) == Internal::ToStrongPtr(Other.Impl->GetInner());
#else
		return false;
#endif	  // #if USE_USD_SDK
	}

	template USDPREGENWRAPPER_API bool FTargetData::operator==(const FTargetData& Other) const;
	template USDPREGENWRAPPER_API bool FTargetData::operator==(const FTargetDataWeak& Other) const;
	template USDPREGENWRAPPER_API bool FTargetDataWeak::operator==(const FTargetData& Other) const;
	template USDPREGENWRAPPER_API bool FTargetDataWeak::operator==(const FTargetDataWeak& Other) const;

#if USE_USD_SDK
	template<typename PtrType>
	FTargetDataBase<PtrType>::FTargetDataBase(const PREGEN_NS::TargetDataRefPtr& InTargetData)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(InTargetData)
		);
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::FTargetDataBase(PREGEN_NS::TargetDataRefPtr&& InTargetData)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(InTargetData))
		);
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::FTargetDataBase(const PREGEN_NS::TargetDataWeakPtr& InTargetData)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(InTargetData)
		);
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::FTargetDataBase(PREGEN_NS::TargetDataWeakPtr&& InTargetData)
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataImpl<PtrType>>(
			Internal::ConvertPtr<PtrType>(MoveTemp(InTargetData))
		);
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::operator PtrType& ()
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::operator const PtrType& () const
	{
		return Impl->GetInner();
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::operator PREGEN_NS::TargetDataRefPtr() const
	{
		return Internal::ToStrongPtr(Impl->GetInner());
	}

	template<typename PtrType>
	FTargetDataBase<PtrType>::operator PREGEN_NS::TargetDataWeakPtr() const
	{
		if constexpr (std::is_same_v<PtrType, PREGEN_NS::TargetDataWeakPtr>)
		{
			return Impl->GetInner();
		}
		else
		{
			return PREGEN_NS::TargetDataWeakPtr{ Impl->GetInner() };
		}
	}
#endif	  // #if USE_USD_SDK

	template<typename PtrType>
	bool FTargetDataBase<PtrType>::IsValid() const
	{
#if USE_USD_SDK
		if (PREGEN_NS::TargetDataRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return Ptr->IsValid();
		}
#endif	  // #if USE_USD_SDK

		return false;
	}

	template<typename PtrType>
	FTargetUid FTargetDataBase<PtrType>::GetUniqueId() const
	{
#if USE_USD_SDK
		if (PREGEN_NS::TargetDataRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return FTargetUid{ Ptr->GetUniqueId() };
		}
#endif	  // #if USE_USD_SDK

		return FTargetUid{};
	}

	template<typename PtrType>
	int32 FTargetDataBase<PtrType>::NumDefinitionEntries() const
	{
#if USE_USD_SDK
		if (PREGEN_NS::TargetDataRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return static_cast<int32>(Ptr->NumDefinitionEntries());
		}
#endif
		return 0;
	}

	template<typename PtrType>
	FTargetDefinitionEntry FTargetDataBase<PtrType>::GetDefinitionEntry(size_t Index) const
	{
#if USE_USD_SDK
		if (PREGEN_NS::TargetDataRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return FTargetDefinitionEntry{ Ptr->GetDefinitionEntry(Index) };
		}
#endif
		return FTargetDefinitionEntry{};
	}

	template<typename PtrType>
	TArray<FTargetDefinitionEntry> FTargetDataBase<PtrType>::GetDefinitionEntries() const
	{
		TArray<FTargetDefinitionEntry> Result;

#if USE_USD_SDK
		if (PREGEN_NS::TargetDataRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			for (const PREGEN_NS::TargetDefinitionEntry& Entry : Ptr->GetDefinitionEntries())
			{
				Result.Add(FTargetDefinitionEntry{ Entry });
			}
		}
#endif	  // #if USE_USD_SDK

		return Result;
	}

	template<typename PtrType>
	TArray<FTargetUid> FTargetDataBase<PtrType>::GetDependencies() const
	{
		TArray<FTargetUid> Result;


#if USE_USD_SDK
		if (PREGEN_NS::TargetDataRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			const std::vector<PREGEN_NS::TargetUid>& Dependencies = Ptr->GetDependencies();
			Result.Reserve(Dependencies.size());

			for (const PREGEN_NS::TargetUid& Dependency : Dependencies)
			{
				Result.Add(FTargetUid{ Dependency });
			}
		}
#endif	  // #if USE_USD_SDK

		return Result;
	}

	template<typename PtrType>
	TArray<FString> FTargetDataBase<PtrType>::GetEncapsulatedDefinitionPaths() const
	{
		TArray<FString> Result;

#if USE_USD_SDK
		if (PREGEN_NS::TargetDataRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			const pxr::SdfPathSet& Paths = Ptr->GetEncapsulatedDefinitionPaths();
			Result.Reserve(static_cast<int32>(Paths.size()));
			for (const pxr::SdfPath& Path : Paths)
			{
				Result.Add(UTF8_TO_TCHAR(Path.GetString().c_str()));
			}
		}
#endif	  // #if USE_USD_SDK

		return Result;
	}

	template<typename PtrType>
	TArray<FString> FTargetDataBase<PtrType>::GetUnencapsulatedDefinitionPaths() const
	{
		TArray<FString> Result;

#if USE_USD_SDK
		if (PREGEN_NS::TargetDataRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			const pxr::SdfPathSet& Paths = Ptr->GetUnencapsulatedDefinitionPaths();
			Result.Reserve(static_cast<int32>(Paths.size()));
			for (const pxr::SdfPath& Path : Paths)
			{
				Result.Add(UTF8_TO_TCHAR(Path.GetString().c_str()));
			}
		}
#endif	  // #if USE_USD_SDK

		return Result;
	}

	template<typename PtrType>
	UE::FSdfLayer FTargetDataBase<PtrType>::GetPermutationOverlay() const
	{
#if USE_USD_SDK
		if (PREGEN_NS::TargetDataRefPtr Ptr = Internal::ToStrongPtr(Impl->GetInner()))
		{
			return UE::FSdfLayer{ Ptr->GetPermutationOverlay() };
		}
#endif	  // #if USE_USD_SDK

		return UE::FSdfLayer{};
	}

#if USE_USD_SDK
	template class FTargetDataBase<PREGEN_NS::TargetDataRefPtr>;
	template class FTargetDataBase<PREGEN_NS::TargetDataWeakPtr>;
#else
	template class FTargetDataBase<FDummyRefPtrType>;
	template class FTargetDataBase<FDummyWeakPtrType>;
#endif	  // #if USE_USD_SDK

	namespace Internal
	{
		class FTargetDataBuilderImpl
		{
		public:
#if USE_USD_SDK
			explicit FTargetDataBuilderImpl(PREGEN_NS::internal::TargetDataBuilder&& InBuilder)
				: Builder(MoveTemp(InBuilder))
				, bConsumed(false)
			{
			}

			TUsdStore<PREGEN_NS::internal::TargetDataBuilder> Builder;
			bool bConsumed;
#endif	  // #if USE_USD_SDK
		};
	}	 // namespace Internal

	FTargetDataBuilder::FTargetDataBuilder(const FTargetUid& Uid)
	{
#if USE_USD_SDK
		// internal::TargetDataBuilder is not default-constructible (it allocates
		// a TargetData up front from the supplied uid), so we use std::optional
		// to defer construction until we are inside the USD allocs scope.
		std::optional<PREGEN_NS::internal::TargetDataBuilder> NativeBuilder;
		{
			FScopedUsdAllocs UsdAllocs;
			NativeBuilder.emplace(static_cast<const PREGEN_NS::TargetUid&>(Uid));
		}

		FScopedUnrealAllocs UnrealAllocs;
		Impl = MakeUnique<Internal::FTargetDataBuilderImpl>(MoveTemp(*NativeBuilder));
#else
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
#endif	  // #if USE_USD_SDK
	}

	FTargetDataBuilder::~FTargetDataBuilder()
	{
		FScopedUnrealAllocs UnrealAllocs;
		Impl.Reset();
	}

	FTargetDataBuilder::FTargetDataBuilder(FTargetDataBuilder&& Other) = default;

	FTargetDataBuilder& FTargetDataBuilder::operator=(FTargetDataBuilder&& Other) = default;

	void FTargetDataBuilder::AddInfo(const FString& DefinitionUid,
	                                  const UE::FSdfPath& ScenePath,
	                                  const TArray<FPermutationOp>& Ops)
	{
#if USE_USD_SDK
		if (!Impl || Impl->bConsumed)
		{
			return;
		}

		FScopedUsdAllocs UsdAllocs;

		PREGEN_NS::PermutationOpVector NativeOps;
		NativeOps.reserve(Ops.Num());
		for (const FPermutationOp& Op : Ops)
		{
			PREGEN_NS::internal::util::SerializedOp Serialized;
			Serialized.typeName = TCHAR_TO_UTF8(*Op.TypeName);
			Serialized.args.reserve(Op.Opargs.Num());
			for (const FPermutationOpArg& Arg : Op.Opargs)
			{
				PREGEN_NS::internal::util::SerializedOpArg ArgOut;
				ArgOut.name = TCHAR_TO_UTF8(*Arg.Name);
				ArgOut.typeName = TCHAR_TO_UTF8(*Arg.TypeName);
				ArgOut.value = TCHAR_TO_UTF8(*Arg.Value);
				Serialized.args.push_back(std::move(ArgOut));
			}

			if (PREGEN_NS::PermutationOpRefPtr NativeOp
				= PREGEN_NS::internal::util::FromSerializedOp(Serialized))
			{
				NativeOps.push_back(std::move(NativeOp));
			}
		}

		const std::string NativeDefnUid = TCHAR_TO_UTF8(*DefinitionUid);

		Impl->Builder.Get().AddInfo(PREGEN_NS::TargetDefinitionEntry{
			NativeDefnUid,
			static_cast<const pxr::SdfPath&>(ScenePath),
			std::move(NativeOps)
		});
#endif	  // #if USE_USD_SDK
	}

	void FTargetDataBuilder::SetDependencies(const TArray<FTargetUid>& Dependencies)
	{
#if USE_USD_SDK
		if (!Impl || Impl->bConsumed)
		{
			return;
		}

		FScopedUsdAllocs UsdAllocs;

		std::vector<PREGEN_NS::TargetUid> NativeDeps;
		NativeDeps.reserve(Dependencies.Num());
		for (const FTargetUid& Dep : Dependencies)
		{
			NativeDeps.push_back(static_cast<const PREGEN_NS::TargetUid&>(Dep));
		}

		Impl->Builder.Get().SetDependencies(std::move(NativeDeps));
#endif	  // #if USE_USD_SDK
	}

	void FTargetDataBuilder::SetEncapsulatedDefinitionPaths(const TArray<FString>& Paths)
	{
#if USE_USD_SDK
		if (!Impl || Impl->bConsumed)
		{
			return;
		}

		FScopedUsdAllocs UsdAllocs;

		// Invalid path strings produce empty SdfPaths; we silently drop them.
		// The path set is diagnostic / optimization data so a malformed entry
		// isn't worth emitting a warning at the wrapper boundary.
		pxr::SdfPathSet NativePaths;
		for (const FString& Path : Paths)
		{
			const std::string Native = TCHAR_TO_UTF8(*Path);
			if (pxr::SdfPath::IsValidPathString(Native))
			{
				NativePaths.insert(pxr::SdfPath(Native));
			}
		}

		Impl->Builder.Get().SetEncapsulatedDefinitionPaths(std::move(NativePaths));
#endif	  // #if USE_USD_SDK
	}

	void FTargetDataBuilder::SetUnencapsulatedDefinitionPaths(const TArray<FString>& Paths)
	{
#if USE_USD_SDK
		if (!Impl || Impl->bConsumed)
		{
			return;
		}

		FScopedUsdAllocs UsdAllocs;

		pxr::SdfPathSet NativePaths;
		for (const FString& Path : Paths)
		{
			const std::string Native = TCHAR_TO_UTF8(*Path);
			if (pxr::SdfPath::IsValidPathString(Native))
			{
				NativePaths.insert(pxr::SdfPath(Native));
			}
		}

		Impl->Builder.Get().SetUnencapsulatedDefinitionPaths(std::move(NativePaths));
#endif	  // #if USE_USD_SDK
	}

	FTargetData FTargetDataBuilder::Build()
	{
#if USE_USD_SDK
		if (!Impl || Impl->bConsumed)
		{
			return FTargetData{};
		}

		FScopedUsdAllocs UsdAllocs;
		PREGEN_NS::TargetDataRefPtr Built = Impl->Builder.Get().Build();
		Impl->bConsumed = true;

		return FTargetData{ Built };
#else
		return FTargetData{};
#endif	  // #if USE_USD_SDK
	}
}	 // namespace UE::UsdPregen