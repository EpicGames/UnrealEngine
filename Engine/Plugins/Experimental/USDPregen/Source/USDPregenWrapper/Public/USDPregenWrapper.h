// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "USDMemory.h"
#include "UsdWrappers/ForwardDeclarations.h"

#include <memory>

#include "USDPregenWrapper.generated.h"

#if USE_USD_SDK

#include "UsdPregen/pregen.h"

PREGEN_NAMESPACE_OPEN_SCOPE

	class StoragePlugin;
	using StoragePluginRefPtr = std::shared_ptr<StoragePlugin>;
	using StoragePluginWeakPtr = std::weak_ptr<StoragePlugin>;

	class SceneTracker;
	using SceneTrackerRefPtr = std::shared_ptr<SceneTracker>;
	using SceneTrackerWeakPtr = std::weak_ptr<SceneTracker>;

	class TargetData;
	using TargetDataRefPtr = std::shared_ptr<TargetData>;
	using TargetDataWeakPtr = std::weak_ptr<TargetData>;

PREGEN_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK

namespace UE
{
	namespace UsdPregen
	{
		template<typename PtrType>
		class FSceneTrackerBase;

#if USE_USD_SDK
		using FSceneTracker = FSceneTrackerBase<PREGEN_NS::SceneTrackerRefPtr>;
		using FSceneTrackerWeak = FSceneTrackerBase<PREGEN_NS::SceneTrackerWeakPtr>;
#else
		using FSceneTracker = FSceneTrackerBase<FDummyRefPtrType>;
		using FSceneTrackerWeak = FSceneTrackerBase<FDummyWeakPtrType>;
#endif
		namespace Internal
		{
			// Helpers for std::shared_ptr and std::weak_ptr conversions

			template<typename T, typename = void>
			struct HasLock : std::false_type {};

			template<typename T>
			struct HasLock<T, std::void_t<decltype(std::declval<T>().lock())>> : std::true_type {};

			template<typename T>
			auto ToStrongPtr(const T& Ptr) -> std::enable_if_t<!HasLock<T>::value, T>
			{
				return Ptr;
			}

			template<typename T>
			auto ToStrongPtr(const T& Ptr) -> decltype(Ptr.lock())
			{
				return Ptr.lock();
			}

			template<typename To, typename From>
			To ConvertPtr(From&& Ptr)
			{
				return To{ std::forward<From>(Ptr) };
			}

			template<typename T>
			std::shared_ptr<T> ConvertPtr(const std::weak_ptr<T>& Ptr)
			{
				return Ptr.lock();
			}

		}	// namespace UE::UsdPregen::Internal
	}	// namespace UE::UsdPregen
}	// namespace UE

UENUM(BlueprintType)
enum class EPregenDiscoveryMode : uint8
{
	AllPermutations,
	ComposedPermutationOnly
};

UENUM(BlueprintType)
enum class EPregenVersionFallbackMode : uint8
{
	None,
	LayerStackFilesAndTimestamps,
	ResolvedLayerStackFilesAndTimestamps
};

UENUM(BlueprintType)
enum class EPregenIdentifierFallbackMode : uint8
{
	None,
	FirstDirectReferenceOrPayload
};

USTRUCT(BlueprintType)
struct FPregenDiscoveryOptions
{
	GENERATED_BODY()

	/// Permutation discovery mode (all variants vs. composed-only).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	EPregenDiscoveryMode DiscoveryMode = EPregenDiscoveryMode::AllPermutations;

	/// Name of the discovery plugin to use. Empty resolves to the built-in
	/// default discovery plugin.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	FString DiscoveryPluginName;

	/// Prefix prepended to every discovered asset definition's name.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	FString DefinitionPrefix;

	/// SdfPath string identifying the prim subtree to limit discovery to.
	/// Empty means "use the discovery plugin's chosen initial path"
	/// (typically the stage's default prim).
	///
	/// Stored as FString for USTRUCT/UPROPERTY compatibility; converted to
	/// pxr::SdfPath at the wrapper -> native boundary.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	FString InitialPath;

	/// UsdGeomImageable purposes to include during discovery. Empty means
	/// "include all purposes".
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	TArray<FString> Purposes;

	/// Variant set names to exclude from permutation enumeration. Use "*"
	/// to exclude all variant sets.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	TArray<FString> ExcludeVariantSets;

	/// Strategy used when an asset has no authored identifier.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	EPregenIdentifierFallbackMode AssetIdentifierFallback = EPregenIdentifierFallbackMode::None;

	/// Strategy used when an asset has no authored version.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	EPregenVersionFallbackMode AssetVersionFallback = EPregenVersionFallbackMode::None;

	bool operator==(const FPregenDiscoveryOptions& Other) const = default;
};

USTRUCT(BlueprintType)
struct FPregenStorageOptions
{
	GENERATED_BODY()

	/// Name of the storage plugin to use. An empty string will use the
	/// built-in JSON plugin ("json_storage").
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	FString StoragePluginName;

	/// Directory where manifest files should be written.
	///
	/// Used by the built-in JsonStoragePlugin. UObject-backed plugins
	/// ignore this since their manifests are UAssets, not files on
	/// disk.
	///
	/// Empty => plugin default (JsonStoragePlugin falls back to the
	/// USDPREGEN_DEFAULT_STORAGE_DIR environment variable, then to
	/// "<HOME>/UE_UsdPregen_Manifests").
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	FString ManifestDir;

	/// ${PLACEHOLDER} template used to build the package sub-path for
	/// generated UAssets. Recognized substitutions are documented on
	/// IStoragePlugin::ResolvePackageSubPathTemplate.
	///
	/// Example: TEXT("assets/${DEFINITION_NAME}/${PERMUTATION_ID}")
	/// produces "assets/tree_02/2559017893".
	///
	/// Empty => plugin default
	/// (IStoragePlugin::DefaultPackageSubPathTemplate).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "USD Pregen")
	FString PackageSubPathTemplate;

	bool operator==(const FPregenStorageOptions& Other) const = default;
};

class USDPregenWrapper
{
public:
	USDPREGENWRAPPER_API static UE::UsdPregen::FSceneTracker CreateSceneTracker(
		const UE::FUsdStage& Stage);

	USDPREGENWRAPPER_API static UE::UsdPregen::FSceneTracker CreateSceneTracker(
		const UE::FUsdStage& Stage,
		const FPregenDiscoveryOptions& Options);
};
