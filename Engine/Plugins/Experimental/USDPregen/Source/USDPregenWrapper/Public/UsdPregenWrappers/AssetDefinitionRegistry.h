// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	class AssetDefinitionRegistry;
}
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	class FExtAssetDefinition;

	class FAssetDefinitionRegistry
	{
	public:

#if USE_USD_SDK
		UE_API explicit FAssetDefinitionRegistry(PREGEN_NS::AssetDefinitionRegistry* InRegistry);

		UE_API operator PREGEN_NS::AssetDefinitionRegistry* ();
		UE_API operator const PREGEN_NS::AssetDefinitionRegistry* () const;
#endif	  // #if USE_USD_SDK

	public:
		UE_API static FAssetDefinitionRegistry GetInstance();

		UE_API FExtAssetDefinition GetDefinition(const FString& UniqueId) const;

#if USE_USD_SDK
	private:
		const PREGEN_NS::AssetDefinitionRegistry* PregenRegistry = nullptr;
#endif
	};
}	 // namespace UE::UsdPregen

#undef UE_API