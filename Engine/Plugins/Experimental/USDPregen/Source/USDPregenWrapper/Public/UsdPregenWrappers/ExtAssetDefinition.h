// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	class ExtAssetDefinition;
}
#endif	  // #if USE_USD_SDK

struct FSdfAssetPath;

namespace UE
{
	class FVtValue;
}

namespace UE::UsdPregen
{
	/**
	 * Minimal PREGEN_NS::ExtAssetDefinition wrapper for Unreal that can be used from no-rtti modules.
	 *
	 * This is a lightweight non-owning wrapper around an immutable ExtAssetDefinition
	 * instance that is owned elsewhere (typically by AssetDefinitionRegistry).
	 */
	class FExtAssetDefinition
	{
	public:
		UE_API FExtAssetDefinition() = default;

#if USE_USD_SDK
		UE_API explicit FExtAssetDefinition(const PREGEN_NS::ExtAssetDefinition* InDefinition);

		UE_API operator const PREGEN_NS::ExtAssetDefinition* () const;
#endif     // #if USE_USD_SDK

		UE_API bool operator==(const FExtAssetDefinition& Other) const;
		UE_API bool operator!=(const FExtAssetDefinition& Other) const;
		UE_API bool operator<(const FExtAssetDefinition& Other) const;

		UE_API explicit operator bool() const;

		UE_API FString GetName() const;
		UE_API FString GetVersion() const;
		UE_API FSdfAssetPath GetIdentifier() const;

		UE_API FString GetUniqueId() const;

		UE_API bool GetMetadata(UE::FVtValue& OutMetadata) const;

		UE_API bool HasMetadata() const;
		UE_API bool HasCustomUniqueId() const;

		UE_API bool IsValid(FString* OutReason = nullptr) const;

	private:
#if USE_USD_SDK
		const PREGEN_NS::ExtAssetDefinition* PregenDefinition = nullptr;
#endif	  // #if USE_USD_SDK
	};

	/// Value-typed snapshot of an ExtAssetDefinition produced for storage.
	///
	/// Captures the authoritative fields so that downstream backends can
	/// persist a definition and re-register it later without depending on
	/// USD headers or RTTI. Round-trip is via the USDA-blob metadata field.
	struct FExtAssetDefinitionSnapshot
	{
		FString UniqueId;
		FString Name;
		FString Version;
		FString IdentifierAuthored;
		FString IdentifierResolved;
		bool bHasCustomUniqueId = false;

		/// Round-tripped metadata VtDictionary, encoded as a USDA-format blob.
		/// Empty when the source definition had no metadata. The encoding
		/// format is an implementation detail and may evolve.
		FString Metadata;

		/// Builds a snapshot from a live FExtAssetDefinition. The Metadata
		/// field is the USDA-encoded blob produced by util.h's
		/// MetadataToUsdaBlob, suitable for round-tripping through
		/// RegisterIntoRegistry().
		USDPREGENWRAPPER_API static FExtAssetDefinitionSnapshot
		From(const FExtAssetDefinition& Defn);

		/// Reconstructs the underlying ExtAssetDefinition and registers it
		/// into the global AssetDefinitionRegistry. Idempotent: returns
		/// true on success or when an identical definition already exists,
		/// false on conflict with a same-UID definition that has different
		/// fields.
		///
		/// On reconstruction, the stored UniqueId is always passed through
		/// the customUniqueId form so it round-trips exactly. As a result
		/// HasCustomUniqueId() will always report true post-load even when
		/// the source had a default-derived UID; the bHasCustomUniqueId
		/// field is informational only.
		USDPREGENWRAPPER_API bool RegisterIntoRegistry() const;
	};
}	 // namespace UE::UsdPregen

#undef UE_API