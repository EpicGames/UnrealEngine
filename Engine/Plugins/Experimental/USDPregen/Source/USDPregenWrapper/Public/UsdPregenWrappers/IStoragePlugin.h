// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"

#define UE_API USDPREGENWRAPPER_API

namespace UE::UsdPregen
{
	class FExtAssetDefinition;
	class FManifest;
	class FTargetUid;

	struct FManifestPayload;
	struct FManifestLoadResult;
	struct FManifestSaveResult;

	class IStoragePlugin
	{
	public:
		UE_API virtual ~IStoragePlugin();

		UE_API virtual FManifestLoadResult LoadManifestPayload(const FTargetUid& TargetUid) = 0;

		UE_API virtual FManifestSaveResult StoreManifestPayload(
			const FTargetUid& TargetUid,
			const FManifestPayload& Payload
		) = 0;

		UE_API virtual FManifestSaveResult PersistManifestPayload(const FTargetUid& TargetUid);

		UE_API virtual FManifestPayload SerializeManifest(const FManifest& Manifest) = 0;

		UE_API virtual FManifest DeserializeManifestPayload(const FManifestPayload& Payload) = 0;

		UE_API virtual FString GetNameForUAsset(
			const FTargetUid& TargetUid,
			const TArray<FExtAssetDefinition>& Definitions,
			const FString& AssetType
		) = 0;

		UE_API virtual FString GetPackageSubPathForUAsset(
			const FTargetUid& TargetUid,
			const TArray<FExtAssetDefinition>& Definitions,
			const FString& AssetType
		) = 0;

		UE_API virtual FString GetPathForManifest(const FTargetUid& TargetUid) = 0;

		/// Default template used by ResolvePackageSubPathTemplate when no
		/// explicit template is provided. Matches the layout the built-in
		/// plugins used before the template became configurable.
		UE_API static const FString& DefaultPackageSubPathTemplate();

		/// Sentinel substituted for placeholder values that resolve to
		/// empty (or to an entirely non-alphanumeric string). E.g.
		/// "versions/${DEFINITION_VERSION}/" with no version becomes
		/// "versions/_/" rather than orphan "versions/".
		UE_API static const FString& EmptyValueSentinel();

		/// Resolve `${PLACEHOLDER}` substitutions in `Template` and
		/// return the final package sub-path. Designed to be reused by
		/// all storage plugin implementations.
		///
		/// Recognized placeholders (all wrapped in `${...}`):
		///   ${DEFINITION_NAME}    - Definitions.Last().GetName()
		///   ${DEFINITION_VERSION} - Definitions.Last().GetVersion()
		///   ${DEFINITION_UID}     - Definitions.Last().GetUniqueId()
		///   ${PERMUTATION_ID}     - TargetUid.GetPermutationUid()
		///   ${ASSET_TYPE}         - the AssetType parameter
		///   ${METADATA:KEY}       - looks up KEY in the leaf definition's
		///                           metadata dictionary (populated from
		///                           USD assetInfo, less the built-in
		///                           keys). Nested dicts are descended
		///                           via colon-separated paths, e.g.
		///                           ${METADATA:nested:subcategory}.
		///                           Non-scalar leaf values (dicts/
		///                           arrays) and missing keys both
		///                           collapse to the empty-value sentinel.
		///
		/// `ExtraSubstitutions` is consulted first and overrides the
		/// built-in placeholders. Unknown placeholders resolve to the
		/// empty-value sentinel ("_").
		///
		/// Each substituted value is sanitized: alnum + `_` pass through;
		/// every other char becomes `_`; empty results become the
		/// sentinel ("_"). Consecutive `/` are collapsed and a single
		/// trailing `/` is trimmed.
		///
		/// Example: Template "assets/${DEFINITION_NAME}/${PERMUTATION_ID}"
		/// with name "chemistry_bottle02" and permutation "2559017893"
		/// resolves to "assets/chemistry_bottle02/2559017893".
		UE_API static FString ResolvePackageSubPathTemplate(
			const FString& Template,
			const FTargetUid& TargetUid,
			const TArray<FExtAssetDefinition>& Definitions,
			const FString& AssetType,
			const TMap<FString, FString>& ExtraSubstitutions = {}
		);
	};
}	 // namespace UE::UsdPregen

#undef UE_API
