// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"

#include "Templates/UniquePtr.h"

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	class JsonStoragePlugin;

	using JsonStoragePluginRefPtr = std::shared_ptr<JsonStoragePlugin>;
}
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	namespace Internal
	{
		template<typename PtrType>
		class FJsonStoragePluginImpl;
	}

	struct FManifestPayload;
	struct FManifestLoadResult;
	struct FManifestSaveResult;

	class FExtAssetDefinition;
	class FManifest;
	class FTargetUid;

	template<typename PtrType>
	class FJsonStoragePluginBase;

#if USE_USD_SDK
	using FJsonStoragePlugin = FJsonStoragePluginBase<PREGEN_NS::JsonStoragePluginRefPtr>;
#else
	using FJsonStoragePlugin = FJsonStoragePluginBase<FDummyRefPtrType>;
#endif	  // #if USE_USD_SDK

	template<typename PtrType>
	class FJsonStoragePluginBase
	{
	public:
		UE_API FJsonStoragePluginBase();

		UE_API explicit FJsonStoragePluginBase(const FPregenStorageOptions& Options);

		UE_API FJsonStoragePluginBase(const FJsonStoragePlugin& Other);
		UE_API FJsonStoragePluginBase(FJsonStoragePlugin&& Other);

		UE_API FJsonStoragePluginBase& operator=(const FJsonStoragePlugin& Other);
		UE_API FJsonStoragePluginBase& operator=(FJsonStoragePlugin&& Other);

		UE_API ~FJsonStoragePluginBase();

		UE_API explicit operator bool() const;

#if USE_USD_SDK
		UE_API explicit FJsonStoragePluginBase(const PREGEN_NS::JsonStoragePluginRefPtr& InPtr);
		UE_API explicit FJsonStoragePluginBase(PREGEN_NS::JsonStoragePluginRefPtr&& InPtr);

		UE_API operator PtrType& ();
		UE_API operator const PtrType& () const;

		UE_API operator PREGEN_NS::JsonStoragePluginRefPtr() const;
#endif	  // #if USE_USD_SDK

	public:
		UE_API FManifestLoadResult LoadManifestPayload(const FTargetUid& TargetUid) const;

		UE_API FManifestSaveResult StoreManifestPayload(
			const FTargetUid& TargetUid,
			const FManifestPayload& Payload
		) const;

		UE_API FManifestSaveResult PersistManifestPayload(const FTargetUid& TargetUid) const;

		UE_API FManifestPayload SerializeManifest(const FManifest& Manifest) const;

		UE_API FManifest DeserializeManifestPayload(const FManifestPayload& Payload) const;

		UE_API FString GetNameForUAsset(
			const FTargetUid& TargetUid,
			const TArray<FExtAssetDefinition>& Definitions,
			const FString& AssetType
		) const;

		UE_API FString GetPackageSubPathForUAsset(
			const FTargetUid& TargetUid,
			const TArray<FExtAssetDefinition>& Definitions,
			const FString& AssetType
		) const;

	private:
		TUniquePtr<Internal::FJsonStoragePluginImpl<PtrType>> Impl;
	};
}	 // namespace UE::UsdPregen

#undef UE_API
