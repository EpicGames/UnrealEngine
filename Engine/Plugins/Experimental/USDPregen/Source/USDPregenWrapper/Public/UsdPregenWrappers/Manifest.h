// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"
#include "UsdPregenWrappers/Target.h"

#include "Templates/UniquePtr.h"

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	struct Product;
	class Manifest;
}
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	namespace Internal
	{
		class FManifestImpl;
	}

	struct FProduct
	{
	public:
		FString UPackagePath;
		FString UClass;
		FString UNodeId;
		FString UsdPrimType;
		FString UsdPrimPath;

	public:
		UE_API FProduct();

		UE_API FProduct(const FProduct& Other);
		UE_API FProduct(FProduct&& Other);

		UE_API FProduct& operator=(const FProduct& Other);
		UE_API FProduct& operator=(FProduct&& Other);

#if USE_USD_SDK
		UE_API explicit FProduct(const PREGEN_NS::Product& InProduct);
		UE_API explicit FProduct(PREGEN_NS::Product&& InProduct);

		UE_API FProduct& operator=(const PREGEN_NS::Product& InProduct);
		UE_API FProduct& operator=(PREGEN_NS::Product&& InProduct);

		UE_API operator PREGEN_NS::Product() const;
#endif	  // #if USE_USD_SDK
	};

	class FManifest
	{
	public:
		UE_API FManifest();

		UE_API FManifest(const FManifest& Other);
		UE_API FManifest(FManifest&& Other);
		UE_API ~FManifest();

		UE_API FManifest& operator=(const FManifest& Other);
		UE_API FManifest& operator=(FManifest&& Other);

		UE_API explicit operator bool() const;

#if USE_USD_SDK
		UE_API explicit FManifest(const PREGEN_NS::Manifest& InManifest);
		UE_API explicit FManifest(PREGEN_NS::Manifest&& InManifest);

		UE_API FManifest& operator=(const PREGEN_NS::Manifest& InManifest);
		UE_API FManifest& operator=(PREGEN_NS::Manifest&& InManifest);

		UE_API operator PREGEN_NS::Manifest& ();
		UE_API operator const PREGEN_NS::Manifest& () const;
#endif	  // #if USE_USD_SDK

	public:
		UE_API FTargetUid GetTargetUid() const;
		UE_API void AddProduct(const FProduct& Product);
		UE_API TArray<FProduct> GetProducts() const;

		/// Attaches the originating target data to this manifest.
		///
		/// Storage backends serialize the attached target data alongside
		/// the products. Pass an empty / invalid FTargetData to clear.
		UE_API void SetTargetData(const FTargetData& TargetData);

		/// Returns the originating target data attached to this manifest.
		///
		/// The returned wrapper evaluates to false when no data is attached.
		UE_API FTargetData GetTargetData() const;

		UE_API bool IsValid() const;

	private:
		TUniquePtr<Internal::FManifestImpl> Impl;
	};
}	 // namespace UE::UsdPregen

#undef UE_API