// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"

#include "Containers/Array.h"

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	struct ManifestPayload;
	struct ManifestLoadResult;
	struct ManifestSaveResult;
	enum class ManifestLoadStatus;
	enum class ManifestSaveStatus;
}
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	enum class EManifestLoadStatus
	{
		Loaded,
		DoesNotExist,
		Error,
	};

	enum class EManifestSaveStatus
	{
		Saved,
		NotSaved,
		Error,
	};

	struct FManifestPayload
	{
	public:
		FString Encoding;
		TArray<uint8> Data;

	public:
		UE_API FManifestPayload();

		UE_API FManifestPayload(const FManifestPayload& Other);
		UE_API FManifestPayload(FManifestPayload&& Other);

		UE_API FManifestPayload& operator=(const FManifestPayload& Other);
		UE_API FManifestPayload& operator=(FManifestPayload&& Other);

#if USE_USD_SDK
		UE_API explicit FManifestPayload(const PREGEN_NS::ManifestPayload& InManifestPayload);
		UE_API explicit FManifestPayload(PREGEN_NS::ManifestPayload&& InManifestPayload);

		UE_API FManifestPayload& operator=(const PREGEN_NS::ManifestPayload& InManifestPayload);
		UE_API FManifestPayload& operator=(PREGEN_NS::ManifestPayload&& InManifestPayload);

		UE_API operator PREGEN_NS::ManifestPayload() const;
#endif	  // #if USE_USD_SDK
	};

	struct FManifestLoadResult
	{
	public:
		EManifestLoadStatus Status = EManifestLoadStatus::Error;
		FManifestPayload Payload;
		FString Message;

	public:
		UE_API FManifestLoadResult();

		UE_API FManifestLoadResult(const FManifestLoadResult& Other);
		UE_API FManifestLoadResult(FManifestLoadResult&& Other);

		UE_API FManifestLoadResult& operator=(const FManifestLoadResult& Other);
		UE_API FManifestLoadResult& operator=(FManifestLoadResult&& Other);

#if USE_USD_SDK
		UE_API explicit FManifestLoadResult(const PREGEN_NS::ManifestLoadResult& InManifestLoadResult);
		UE_API explicit FManifestLoadResult(PREGEN_NS::ManifestLoadResult&& InManifestLoadResult);

		UE_API FManifestLoadResult& operator=(const PREGEN_NS::ManifestLoadResult& InManifestLoadResult);
		UE_API FManifestLoadResult& operator=(PREGEN_NS::ManifestLoadResult&& InManifestLoadResult);

		UE_API operator PREGEN_NS::ManifestLoadResult() const;
#endif	  // #if USE_USD_SDK
	};

	struct FManifestSaveResult
	{
	public:
		EManifestSaveStatus Status = EManifestSaveStatus::Error;
		FString Message;

	public:
		UE_API FManifestSaveResult();

		UE_API FManifestSaveResult(const FManifestSaveResult& Other);
		UE_API FManifestSaveResult(FManifestSaveResult&& Other);

		UE_API FManifestSaveResult& operator=(const FManifestSaveResult& Other);
		UE_API FManifestSaveResult& operator=(FManifestSaveResult&& Other);

#if USE_USD_SDK
		UE_API explicit FManifestSaveResult(const PREGEN_NS::ManifestSaveResult& InManifestSaveResult);
		UE_API explicit FManifestSaveResult(PREGEN_NS::ManifestSaveResult&& InManifestSaveResult);

		UE_API FManifestSaveResult& operator=(const PREGEN_NS::ManifestSaveResult& InManifestSaveResult);
		UE_API FManifestSaveResult& operator=(PREGEN_NS::ManifestSaveResult&& InManifestSaveResult);

		UE_API operator PREGEN_NS::ManifestSaveResult() const;
#endif	  // #if USE_USD_SDK
	};
}	 // namespace UE::UsdPregen

#undef UE_API