// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"
#include "UsdPregenWrappers/Target.h"

#include "Templates/UniquePtr.h"

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	class SceneDiscovery;
}
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FSdfPath;
}

struct FPregenDiscoveryOptions;

namespace UE::UsdPregen
{
	namespace Internal
	{
		class FSceneDiscoveryImpl;
	}

	class FSceneDiscovery
	{
	public:
		using ResultMap = TMap<UE::FSdfPath, TArray<FTargetUid>>;

	public:
		UE_API explicit FSceneDiscovery(const UE::FUsdStage& Stage);
		UE_API FSceneDiscovery(const UE::FUsdStage& Stage, const FPregenDiscoveryOptions& Options);

		UE_API FSceneDiscovery(const FSceneDiscovery& Other);
		UE_API FSceneDiscovery(FSceneDiscovery&& Other);
		UE_API ~FSceneDiscovery();

		UE_API FSceneDiscovery& operator=(const FSceneDiscovery& Other);
		UE_API FSceneDiscovery& operator=(FSceneDiscovery&& Other);

#if USE_USD_SDK
		UE_API explicit FSceneDiscovery(const PREGEN_NS::SceneDiscovery& InSceneDiscovery);
		UE_API explicit FSceneDiscovery(PREGEN_NS::SceneDiscovery&& InSceneDiscovery);

		UE_API FSceneDiscovery& operator=(const PREGEN_NS::SceneDiscovery& InSceneDiscovery);
		UE_API FSceneDiscovery& operator=(PREGEN_NS::SceneDiscovery&& InSceneDiscovery);

		UE_API operator PREGEN_NS::SceneDiscovery& ();
		UE_API operator const PREGEN_NS::SceneDiscovery& () const;
#endif	  // #if USE_USD_SDK

	public:
		UE_API FTargetData GetTargetData(const FTargetUid& TargetUid);

		UE_API bool TraverseAndFindTargets(ResultMap& Results);

		UE_API bool SaveDiscoveryData(const FString& Filename);

	private:
		TUniquePtr<Internal::FSceneDiscoveryImpl> Impl;
	};
}	 // namespace UE::UsdPregen

#undef UE_API