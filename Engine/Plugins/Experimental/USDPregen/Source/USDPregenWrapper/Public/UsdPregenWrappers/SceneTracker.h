// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"
#include "UsdPregenWrappers/Target.h"

#include "Templates/UniquePtr.h"

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	class SceneTracker;
	class TrackedPrim;

	using SceneTrackerRefPtr = std::shared_ptr<SceneTracker>;
	using SceneTrackerWeakPtr = std::weak_ptr<SceneTracker>;
}
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FSdfPath;
	class FUsdPrim;
}

namespace UE::UsdPregen
{
	namespace Internal
	{
		template<typename PtrType>
		class FSceneTrackerImpl;

		class FTrackedPrimImpl;
	}

	class FTrackedPrim
	{
	public:
		UE_API FTrackedPrim();
		UE_API FTrackedPrim(FTrackedPrim&& Other);
		UE_API ~FTrackedPrim();

		UE_API FTrackedPrim& operator=(FTrackedPrim&& Other);

		FTrackedPrim(const FTrackedPrim&) = delete;
		FTrackedPrim& operator=(const FTrackedPrim&) = delete;

#if USE_USD_SDK
		UE_API explicit FTrackedPrim(PREGEN_NS::TrackedPrim&& InTrackedPrim);

		UE_API FTrackedPrim& operator=(PREGEN_NS::TrackedPrim&& InTrackedPrim);
#endif	  // #if USE_USD_SDK

	public:
		UE_API bool HasUnprocessedPermutations() const;
		UE_API bool PrepareNextPermutation();

	private:
		TUniquePtr<Internal::FTrackedPrimImpl> Impl;
	};

	template<typename PtrType>
	class FSceneTrackerBase
	{
	public:
		UE_API FSceneTrackerBase();

		UE_API FSceneTrackerBase(const FSceneTracker& Other);
		UE_API FSceneTrackerBase(FSceneTracker&& Other);
		UE_API FSceneTrackerBase(const FSceneTrackerWeak& Other);
		UE_API FSceneTrackerBase(FSceneTrackerWeak&& Other);

		UE_API FSceneTrackerBase& operator=(const FSceneTracker& Other);
		UE_API FSceneTrackerBase& operator=(FSceneTracker&& Other);
		UE_API FSceneTrackerBase& operator=(const FSceneTrackerWeak& Other);
		UE_API FSceneTrackerBase& operator=(FSceneTrackerWeak&& Other);

		UE_API ~FSceneTrackerBase();

		UE_API explicit operator bool() const;

		template<typename OtherPtrType>
		UE_API bool operator==(const FSceneTrackerBase<OtherPtrType>& Other) const;

#if USE_USD_SDK
		UE_API explicit FSceneTrackerBase(const PREGEN_NS::SceneTrackerRefPtr& InSceneTracker);
		UE_API explicit FSceneTrackerBase(PREGEN_NS::SceneTrackerRefPtr&& InSceneTracker);
		UE_API explicit FSceneTrackerBase(const PREGEN_NS::SceneTrackerWeakPtr& InSceneTracker);
		UE_API explicit FSceneTrackerBase(PREGEN_NS::SceneTrackerWeakPtr&& InSceneTracker);

		UE_API operator PtrType& ();
		UE_API operator const PtrType& () const;

		UE_API operator PREGEN_NS::SceneTrackerRefPtr() const;
		UE_API operator PREGEN_NS::SceneTrackerWeakPtr() const;
#endif	  // #if USE_USD_SDK

	public:
		UE_API FPregenDiscoveryOptions GetOptions() const;

		UE_API FTargetData GetTargetData(const FTargetUid& TargetId) const;

		UE_API bool HasErrors() const;

		UE_API bool SaveDataLayer(const FString& Filename) const;

		UE_API FTrackedPrim StartTrackingPrim(const UE::FUsdPrim& Prim);

		UE_API void SetTargetCreatedCallback(
			TFunction<void(const UE::FSdfPath&, const FTargetUid&)> Callback
		);

		UE_API void RemoveTargetCreatedCallback();

	private:
		friend FSceneTracker;
		friend FSceneTrackerWeak;

		TUniquePtr<Internal::FSceneTrackerImpl<PtrType>> Impl;
	};
}	 // namespace UE::UsdPregen

#undef UE_API