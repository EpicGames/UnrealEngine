// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"

#include "Templates/UniquePtr.h"

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	class StoragePlugin;

	using StoragePluginRefPtr = std::shared_ptr<StoragePlugin>;
	using StoragePluginWeakPtr = std::weak_ptr<StoragePlugin>;
}
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	struct FManifestPayload;
	struct FManifestLoadResult;
	struct FManifestSaveResult;

	class FExtAssetDefinition;
	class FManifest;
	class FTargetUid;

	namespace Internal
	{
		template<typename PtrType>
		class FStoragePluginImpl;
	}

	template<typename PtrType>
	class FStoragePluginBase;

#if USE_USD_SDK
	using FStoragePlugin = FStoragePluginBase<PREGEN_NS::StoragePluginRefPtr>;
	using FStoragePluginWeak = FStoragePluginBase<PREGEN_NS::StoragePluginWeakPtr>;
#else
	using FStoragePlugin = FStoragePluginBase<FDummyRefPtrType>;
	using FStoragePluginWeak = FStoragePluginBase<FDummyWeakPtrType>;
#endif	  // #if USE_USD_SDK

	template<typename PtrType>
	class FStoragePluginBase
	{
	public:
		UE_API FStoragePluginBase();

		UE_API FStoragePluginBase(const FStoragePlugin& Other);
		UE_API FStoragePluginBase(FStoragePlugin&& Other);
		UE_API FStoragePluginBase(const FStoragePluginWeak& Other);
		UE_API FStoragePluginBase(FStoragePluginWeak&& Other);

		UE_API FStoragePluginBase& operator=(const FStoragePlugin& Other);
		UE_API FStoragePluginBase& operator=(FStoragePlugin&& Other);
		UE_API FStoragePluginBase& operator=(const FStoragePluginWeak& Other);
		UE_API FStoragePluginBase& operator=(FStoragePluginWeak&& Other);

		UE_API ~FStoragePluginBase();

		UE_API explicit operator bool() const;

		template<typename OtherPtrType>
		UE_API bool operator==(const FStoragePluginBase<OtherPtrType>& Other) const;

#if USE_USD_SDK
		UE_API explicit FStoragePluginBase(const PREGEN_NS::StoragePluginRefPtr& InStorage);
		UE_API explicit FStoragePluginBase(PREGEN_NS::StoragePluginRefPtr&& InStorage);
		UE_API explicit FStoragePluginBase(const PREGEN_NS::StoragePluginWeakPtr& InStorage);
		UE_API explicit FStoragePluginBase(PREGEN_NS::StoragePluginWeakPtr&& InStorage);

		UE_API operator PtrType& ();
		UE_API operator const PtrType& () const;

		UE_API operator PREGEN_NS::StoragePluginRefPtr() const;
		UE_API operator PREGEN_NS::StoragePluginWeakPtr() const;
#endif	  // #if USE_USD_SDK

	public:
		UE_API bool IsValid() const;

		UE_API bool Initialize() const;

		UE_API bool Shutdown() const;

		UE_API FManifestLoadResult LoadManifestPayload(const FTargetUid& TargetUid) const;

		UE_API FManifestSaveResult StoreManifestPayload(
			const FTargetUid& TargetUid,
			const FManifestPayload& Payload
		) const;

		UE_API FManifestSaveResult PersistManifestPayload(const FTargetUid& TargetUid) const;

		UE_API FManifestPayload SerializeManifest(const FManifest& Manifest) const;

		UE_API FManifest DeserializeManifestPayload(const FManifestPayload& Payload) const;

		UE_API FString GetNameForUAsset(
			const FTargetUid& TargetUid,
			const TArray<FExtAssetDefinition>& Definitions,
			const FString& AssetType
		) const;

		UE_API FString GetPackageSubPathForUAsset(
			const FTargetUid& TargetUid,
			const TArray<FExtAssetDefinition>& Definitions,
			const FString& AssetType
		) const;

		UE_API FString GetPathForManifest(const FTargetUid& TargetUid) const;

	private:
		friend FStoragePlugin;
		friend FStoragePluginWeak;

		TUniquePtr<Internal::FStoragePluginImpl<PtrType>> Impl;
	};
}	 // namespace UE::UsdPregen

#undef UE_API