// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"
#include "UsdPregenWrappers/IStoragePlugin.h"

#include "Templates/UniquePtr.h"

#if USE_USD_SDK
#include "UsdPregen/storagePlugin.h"
#include <memory>
#include <string>
#include <vector>
#endif	  // #if USE_USD_SDK

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	class ExtAssetDefinition;
	class StoragePlugin;

	using StoragePluginRefPtr = std::shared_ptr<StoragePlugin>;
}
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
#if USE_USD_SDK
	class FStoragePluginAdapter : public PREGEN_NS::StoragePlugin
	{
	public:
		UE_API explicit FStoragePluginAdapter(
			const TSharedRef<IStoragePlugin, ESPMode::ThreadSafe>& InStoragePlugin
		);

		UE_API virtual ~FStoragePluginAdapter() override;

		UE_API virtual PREGEN_NS::ManifestLoadResult LoadManifestPayload(
			const PREGEN_NS::TargetUid& TargetUid
		) override;

		UE_API virtual PREGEN_NS::ManifestSaveResult StoreManifestPayload(
			const PREGEN_NS::TargetUid& TargetUid,
			const PREGEN_NS::ManifestPayload& Payload
		) override;

		UE_API virtual PREGEN_NS::ManifestSaveResult PersistManifestPayload(
			const PREGEN_NS::TargetUid& TargetUid
		) override;

		UE_API virtual PREGEN_NS::ManifestPayload SerializeManifest(
			const PREGEN_NS::Manifest& Manifest
		) override;

		UE_API virtual PREGEN_NS::Manifest DeserializeManifestPayload(
			const PREGEN_NS::ManifestPayload& Payload
		) override;

		UE_API virtual std::string GetNameForUAsset(
			const PREGEN_NS::TargetUid& TargetUid,
			const std::vector<const PREGEN_NS::ExtAssetDefinition*>& ExtAssetDefinitions,
			const std::string& AssetType
		) override;

		UE_API virtual std::string GetPackageSubPathForUAsset(
			const PREGEN_NS::TargetUid& TargetUid,
			const std::vector<const PREGEN_NS::ExtAssetDefinition*>& ExtAssetDefinitions,
			const std::string& AssetType
		) override;

		UE_API virtual std::string GetPathForManifest(
			const PREGEN_NS::TargetUid& TargetUid
		) override;

	private:
		TSharedRef<IStoragePlugin, ESPMode::ThreadSafe> StoragePlugin;
	};

	UE_API PREGEN_NS::StoragePluginRefPtr MakeStoragePluginAdapter(
		const TSharedRef<IStoragePlugin, ESPMode::ThreadSafe>& InStoragePlugin
	);
#endif	  // #if USE_USD_SDK
}	 // namespace UE::UsdPregen

#undef UE_API
