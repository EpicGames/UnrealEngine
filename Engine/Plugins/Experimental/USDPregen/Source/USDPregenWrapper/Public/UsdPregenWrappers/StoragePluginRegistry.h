// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"
#include "UsdPregenWrappers/StoragePlugin.h"

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	class StoragePluginRegistry;
}
#endif	  // #if USE_USD_SDK

namespace UE::UsdPregen
{
	class IStoragePlugin;

	using FCreateStoragePluginFunction =
		TFunction<TSharedRef<IStoragePlugin, ESPMode::ThreadSafe>(const FPregenStorageOptions& Options)>;

	class FStoragePluginRegistry
	{
	public:
		UE_API FStoragePluginRegistry() = default;

#if USE_USD_SDK
		UE_API explicit FStoragePluginRegistry(PREGEN_NS::StoragePluginRegistry* InRegistry);

		UE_API operator PREGEN_NS::StoragePluginRegistry* ();
		UE_API operator const PREGEN_NS::StoragePluginRegistry* () const;
#endif	  // #if USE_USD_SDK

	public:
		UE_API static FStoragePluginRegistry GetInstance();

		UE_API bool RegisterFactory(
			const FString& Name,
			FCreateStoragePluginFunction Factory
		);

		UE_API FStoragePlugin Create(
			const FPregenStorageOptions& Options
		) const;

#if USE_USD_SDK
	private:
		PREGEN_NS::StoragePluginRegistry* PregenRegistry = nullptr;
#endif	  // #if USE_USD_SDK
	};
}	 // namespace UE::UsdPregen

#undef UE_API
