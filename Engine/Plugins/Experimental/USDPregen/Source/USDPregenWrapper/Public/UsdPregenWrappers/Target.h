// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPregenWrapper.h"

#include "Templates/UniquePtr.h"

#define UE_API USDPREGENWRAPPER_API

#if USE_USD_SDK
namespace PREGEN_NS
{
	class TargetUid;
	class TargetDefinitionEntry;
	class TargetData;

	using TargetDataRefPtr = std::shared_ptr<TargetData>;
	using TargetDataWeakPtr = std::weak_ptr<TargetData>;
}
#endif	  // #if USE_USD_SDK

namespace UE
{
	class FSdfPath;
}

namespace UE::UsdPregen
{
	class FExtAssetDefinition;

	namespace Internal
	{
		class FTargetUidImpl;
		class FTargetDefinitionEntryImpl;

		template<typename PtrType>
		class FTargetDataImpl;
	}

	template<typename PtrType>
	class FTargetDataBase;

#if USE_USD_SDK
	using FTargetData = FTargetDataBase<PREGEN_NS::TargetDataRefPtr>;
	using FTargetDataWeak = FTargetDataBase<PREGEN_NS::TargetDataWeakPtr>;
#else
	using FTargetData = FTargetDataBase<FDummyRefPtrType>;
	using FTargetDataWeak = FTargetDataBase<FDummyWeakPtrType>;
#endif	  // #if USE_USD_SDK

	class FTargetUid
	{
	public:
		UE_API FTargetUid();

		UE_API FTargetUid(const FTargetUid& Other);
		UE_API FTargetUid(FTargetUid&& Other);
		UE_API ~FTargetUid();

		UE_API FTargetUid& operator=(const FTargetUid& Other);
		UE_API FTargetUid& operator=(FTargetUid&& Other);

		UE_API explicit FTargetUid(const FString& DefinitionUid);
		UE_API FTargetUid(const FString& DefinitionUid, const FString& PermutationUid);

		UE_API bool operator==(const FTargetUid& Other) const;
		UE_API bool operator!=(const FTargetUid& Other) const;
		UE_API bool operator<(const FTargetUid& Other) const;

		UE_API explicit operator bool() const;

#if USE_USD_SDK
		UE_API explicit FTargetUid(const PREGEN_NS::TargetUid& InTargetUid);
		UE_API explicit FTargetUid(PREGEN_NS::TargetUid&& InTargetUid);

		UE_API FTargetUid& operator=(const PREGEN_NS::TargetUid& InTargetUid);
		UE_API FTargetUid& operator=(PREGEN_NS::TargetUid&& InTargetUid);

		UE_API operator PREGEN_NS::TargetUid& ();
		UE_API operator const PREGEN_NS::TargetUid& () const;
#endif	  // #if USE_USD_SDK

	public:
		UE_API FString GetDefinitionUid() const;
		UE_API FString GetPermutationUid() const;
		UE_API bool HasPermutationUid() const;
		UE_API FString GetString() const;
		UE_API bool IsValid() const;

	private:
		TUniquePtr<Internal::FTargetUidImpl> Impl;
	};

	UE_API uint32 GetTypeHash(const FTargetUid& TargetUid);

	/// Argument extracted from an op's "opargs:*" attribute namespace.
	///
	/// Keeps the Sdf value type name alongside the string-encoded value so
	/// the op can be reconstructed faithfully on the deserialize side.
	struct FPermutationOpArg
	{
		/// Argument name with the "opargs:" prefix removed.
		FString Name;

		/// Sdf value type name (e.g. "string", "token").
		FString TypeName;

		/// String-encoded default value.
		FString Value;
	};

	/// Lightweight wrapper view of a single PermutationOp.
	///
	/// Captures the prim-spec type name authored by the underlying op's
	/// Serialize() implementation plus its "opargs:*" attributes. Mirrors
	/// the Core SerializedOp shape one-for-one so wrapper consumers can 
	/// round-trip an op without depending on Core or USD headers directly.
	struct FPermutationOp
	{
		/// Op type name (e.g. "UsdVariantSelectionOp", "UsdInheritOp",
		/// "UsdSchemaApplyOp"). Matches the value written by the underlying
		/// op's Serialize() override.
		FString TypeName;

		/// Argument list extracted from the op's "opargs:" namespace.
		TArray<FPermutationOpArg> Opargs;
	};

	class FTargetDefinitionEntry
	{
	public:
		UE_API FTargetDefinitionEntry();

		UE_API FTargetDefinitionEntry(const FTargetDefinitionEntry& Other);
		UE_API FTargetDefinitionEntry(FTargetDefinitionEntry&& Other);
		UE_API ~FTargetDefinitionEntry();

		UE_API FTargetDefinitionEntry& operator=(const FTargetDefinitionEntry& Other);
		UE_API FTargetDefinitionEntry& operator=(FTargetDefinitionEntry&& Other);

		UE_API FTargetDefinitionEntry(const FString& DefinitionUid, const UE::FSdfPath& ScenePath);

#if USE_USD_SDK
		UE_API explicit FTargetDefinitionEntry(const PREGEN_NS::TargetDefinitionEntry& InTargetDefinitionEntry);
		UE_API explicit FTargetDefinitionEntry(PREGEN_NS::TargetDefinitionEntry&& InTargetDefinitionEntry);

		UE_API FTargetDefinitionEntry& operator=(const PREGEN_NS::TargetDefinitionEntry& InTargetDefinitionEntry);
		UE_API FTargetDefinitionEntry& operator=(PREGEN_NS::TargetDefinitionEntry&& InTargetDefinitionEntry);

		UE_API operator PREGEN_NS::TargetDefinitionEntry& ();
		UE_API operator const PREGEN_NS::TargetDefinitionEntry& () const;
#endif	  // #if USE_USD_SDK

	public:
		UE_API FExtAssetDefinition GetDefinition() const;
		UE_API UE::FSdfPath GetScenePath() const;

		/// Returns the lightweight permutation ops attached to this info.
		///
		/// May be empty when no permutation ops are associated with this scope.
		UE_API TArray<FPermutationOp> GetPermutationOps() const;

	private:
		TUniquePtr<Internal::FTargetDefinitionEntryImpl> Impl;
	};

	template<typename PtrType>
	class FTargetDataBase
	{
	public:
		UE_API FTargetDataBase();

		UE_API FTargetDataBase(const FTargetData& Other);
		UE_API FTargetDataBase(FTargetData&& Other);
		UE_API FTargetDataBase(const FTargetDataWeak& Other);
		UE_API FTargetDataBase(FTargetDataWeak&& Other);

		UE_API FTargetDataBase& operator=(const FTargetData& Other);
		UE_API FTargetDataBase& operator=(FTargetData&& Other);
		UE_API FTargetDataBase& operator=(const FTargetDataWeak& Other);
		UE_API FTargetDataBase& operator=(FTargetDataWeak&& Other);

		UE_API ~FTargetDataBase();

		UE_API explicit operator bool() const;

		template<typename OtherPtrType>
		UE_API bool operator==(const FTargetDataBase<OtherPtrType>& Other) const;

#if USE_USD_SDK
		UE_API explicit FTargetDataBase(const PREGEN_NS::TargetDataRefPtr& InTargetData);
		UE_API explicit FTargetDataBase(PREGEN_NS::TargetDataRefPtr&& InTargetData);
		UE_API explicit FTargetDataBase(const PREGEN_NS::TargetDataWeakPtr& InTargetData);
		UE_API explicit FTargetDataBase(PREGEN_NS::TargetDataWeakPtr&& InTargetData);

		UE_API operator PtrType& ();
		UE_API operator const PtrType& () const;

		UE_API operator PREGEN_NS::TargetDataRefPtr() const;
		UE_API operator PREGEN_NS::TargetDataWeakPtr() const;
#endif	  // #if USE_USD_SDK

	public:
		UE_API bool IsValid() const;

		UE_API FTargetUid GetUniqueId() const;
		UE_API int32 NumDefinitionEntries() const;
		UE_API FTargetDefinitionEntry GetDefinitionEntry(size_t Index) const;
		UE_API TArray<FTargetDefinitionEntry> GetDefinitionEntries() const;
		UE_API TArray<FTargetUid> GetDependencies() const;

		/// Returns the encapsulated definition paths as scene-path strings.
		UE_API TArray<FString> GetEncapsulatedDefinitionPaths() const;

		/// Returns the unencapsulated definition paths as scene-path strings.
		UE_API TArray<FString> GetUnencapsulatedDefinitionPaths() const;

		UE_API UE::FSdfLayer GetPermutationOverlay() const;

	private:
		friend FTargetData;
		friend FTargetDataWeak;

		TUniquePtr<Internal::FTargetDataImpl<PtrType>> Impl;
	};

	namespace Internal
	{
		class FTargetDataBuilderImpl;
	}

	/// Builder for reconstructing an FTargetData from snapshot fields.
	///
	/// Used by storage backends to rebuild a target
	/// data from serialized form without having to depend on Core or USD
	/// headers directly. Internally creates the underlying TargetData via
	/// Core's internal::TargetDataBuilder.
	class FTargetDataBuilder
	{
	public:
		UE_API explicit FTargetDataBuilder(const FTargetUid& Uid);
		UE_API ~FTargetDataBuilder();

		FTargetDataBuilder(const FTargetDataBuilder&) = delete;
		FTargetDataBuilder& operator=(const FTargetDataBuilder&) = delete;

		UE_API FTargetDataBuilder(FTargetDataBuilder&& Other);
		UE_API FTargetDataBuilder& operator=(FTargetDataBuilder&& Other);

		/// Adds a TargetDefinitionEntry entry with its associated permutation ops.
		UE_API void AddInfo(const FString& DefinitionUid,
		                    const UE::FSdfPath& ScenePath,
		                    const TArray<FPermutationOp>& Ops);

		UE_API void SetDependencies(const TArray<FTargetUid>& Dependencies);

		/// Replaces the encapsulated definition path set. Invalid path
		/// strings are skipped with a warning.
		UE_API void SetEncapsulatedDefinitionPaths(const TArray<FString>& Paths);

		/// Replaces the unencapsulated definition path set. Invalid path
		/// strings are skipped with a warning.
		UE_API void SetUnencapsulatedDefinitionPaths(const TArray<FString>& Paths);

		/// Finalizes construction and returns the populated FTargetData.
		///
		/// The builder is consumed by this call - subsequent invocations
		/// return an empty FTargetData.
		UE_API FTargetData Build();

	private:
		TUniquePtr<Internal::FTargetDataBuilderImpl> Impl;
	};
}	 // namespace UE::UsdPregen

#undef UE_API