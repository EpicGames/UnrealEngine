// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDPregenWrapper : ModuleRules
	{
		public USDPregenWrapper(ReadOnlyTargetRules Target) : base(Target)
		{            
			bUseRTTI = true;
			bLegalToDistributeObjectCode = true;
			bUseUnity = true;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"USDPregenCore",
					"UnrealUSDWrapper"
				}
			);

			// Grants Wrapper's private compile units access to USDPregenCore's
			// Private/ headers. Wrapper is by design the UE-facing adapter for
			// Core's wire-format helpers (util.h's MetadataToUsdaBlob and
			// SerializedOp (de)serialization), and that bridge can't live in
			// Core itself (Core has no UStruct knowledge). Scoping the friend
			// access to this single module keeps Core's Private/ surface
			// inaccessible to UStorage, Interchange, Py, and downstream
			// projects.
			PrivateIncludePaths.AddRange(
				new string[]
				{
					Path.Combine(ModuleDirectory, "..", "USDPregenCore", "Private")
				}
			);

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}
