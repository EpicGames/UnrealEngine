{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "USDPregen",
	"Description": "Library to assist with pre-generating USD-based content.",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "USDPregenCore",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		},
        {
			"Name": "USDPregenHttpWorker",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "USDPregenPy",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		},
        {
			"Name": "USDPregenWrapper",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "USDPregenInterchange",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "USDPregenInterchangeEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "USDPregenUObjectStorage",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "USDCore",
			"Enabled": true
		},
		{
			"Name": "Interchange",
			"Enabled": true
		},
		{
			"Name": "InterchangeOpenUSD",
			"Enabled": true
		}
	]
}